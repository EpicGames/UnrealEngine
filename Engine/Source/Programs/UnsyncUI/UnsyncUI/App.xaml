<!--Copyright Epic Games, Inc. All Rights Reserved.-->

<Application x:Class="UnsyncUI.App"
             xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
			 xmlns:s="clr-namespace:System;assembly=mscorlib"
             xmlns:local="clr-namespace:UnsyncUI"
             StartupUri="MainWindow.xaml">
    <Application.Resources>
        <Style TargetType="local:MainWindow">
            <Setter Property="Margin" Value="4" />
        </Style>
        <Style TargetType="Button">
            <Setter Property="Margin" Value="4" />
        </Style>
        <Style TargetType="TextBox">
            <Setter Property="Margin" Value="4" />
        </Style>
		<Style TargetType="ProgressBar">
			<Setter Property="Height" Value="4" />
			<Setter Property="Margin" Value="4" />
		</Style>
		
		<Visibility x:Key="Hidden">Hidden</Visibility>
		<Visibility x:Key="Collapsed">Collapsed</Visibility>
		<Visibility x:Key="Visible">Visible</Visibility>
		
		<s:Boolean x:Key="True">True</s:Boolean>
		<s:Boolean x:Key="False">False</s:Boolean>
		
		<local:AllConverter x:Key="BoolConv_All" TrueValue="{StaticResource True}" FalseValue="{StaticResource False}"/>

		<local:AllConverter x:Key="VisConv_All_Collapsed" TrueValue="{StaticResource Visible}" FalseValue="{StaticResource Collapsed}"/>
		<local:NotConverter x:Key="NotConv"/>
		<local:VisibilityConverter x:Key="VisConv_Collapsed" FalseState="Collapsed" />
		<local:VisibilityConverter x:Key="VisConv_Collapsed_Inv" TrueState="Collapsed" FalseState="Visible" />
		<local:VisibilityConverter x:Key="VisConv_Hidden" FalseState="Hidden" />
		<local:ObjectToBoolConverter x:Key="IsObjectNotNull"/>	
		
	</Application.Resources>
</Application>
