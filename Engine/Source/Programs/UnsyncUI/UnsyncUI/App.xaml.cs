// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Text.Json;
using System.Threading;
using System.Windows;
using System.Xml.Linq;

namespace UnsyncUI
{
	/// <summary>
	/// Interaction logic for App.xaml
	/// </summary>
	public partial class App : Application
    {
		public App() : base()
		{
			Dispatcher.UnhandledException += (sender, args) => UnhandledException(sender, args.Exception.ToString());
			AppDomain.CurrentDomain.UnhandledException += (sender, args) => UnhandledException(sender, args.ExceptionObject.ToString());
		}

		public new static App Current => Application.Current as App;

		public Config Config { get; private set; }
		public string UnsyncPath { get; private set; }
		public UserPreferences UserConfig { get; private set; }
		public string DefaultSearchTerms { get; private set; } = "";

		internal bool EnableExperimentalFeatures = false;
		internal bool EnableUserAuthentication = false;
		internal bool UnsyncSupportsStdout = false;

		internal Dictionary<string, string> Variables = new Dictionary<string, string>();

		internal string ApplicationLog { get; private set; } = "";

		internal void ClearApplicationLog()
		{
			Dispatcher.InvokeAsync(delegate
			{
				ApplicationLog = "";
				var model = MainWindow.DataContext as MainWindowModel;
				if (model != null)
				{
					model.OnLogUpdated();
				}
			});
		}

		internal void LogError(string message)
		{
			LogMessage("ERROR: " + message);
		}

		internal void LogMessage(string message)
		{
			LogDebug(message);

			Dispatcher.InvokeAsync(delegate
			{
				ApplicationLog += message + "\n";
				var model = MainWindow.DataContext as MainWindowModel;
				if (model != null)
				{
					model.OnLogUpdated();
				}
			});
		}

		internal void LogDebug(string message)
		{
			Debug.WriteLine(message);
		}

		protected override void OnStartup(StartupEventArgs e)
		{
			string configFile = e.Args.Length > 0 ? e.Args[0] : "unsyncui.xml";

			List<string> VariableKeys = new List<string> { "ProjectFile", "ProjectName", "ProjectDir" };

			for (int i=1; i<e.Args.Length; ++i)
			{
				bool isLast = i + 1 == e.Args.Length;
				var arg = e.Args[i];
				if (arg == "--search" && !isLast)
				{
					DefaultSearchTerms = e.Args[i + 1];
					++i;
				}
				else if (arg == "--experimental")
				{
					EnableExperimentalFeatures = true;
				}
				else
				{
					string argLower = arg.ToLower();
					foreach (string key in VariableKeys)
					{
						string keyLower = key.ToLower();
						if ((argLower == $"--{keyLower}" || argLower == $"-{keyLower}") && !isLast)
						{
							string value = e.Args[i + 1];
							Variables.Add(key, value);
							++i;
						}
					}
				}
			}

			// Derive project name from project file if one is not specified explicitly
			if (!Variables.ContainsKey("ProjectName") && Variables.ContainsKey("ProjectFile"))
			{
				string ProjectFile = Variables["ProjectFile"];
				string ProjectName = Path.GetFileNameWithoutExtension(ProjectFile);
				if (!string.IsNullOrWhiteSpace(ProjectName))
				{
					Variables["ProjectName"] = ProjectName;
				}
			}

			// Derive project directory from project file if one is not specified explicitly
			if (!Variables.ContainsKey("ProjectDir") && Variables.ContainsKey("ProjectFile"))
			{
				string ProjectFile = Variables["ProjectFile"];
				string ProjectDir = Path.GetDirectoryName(ProjectFile);
				if (!string.IsNullOrWhiteSpace(ProjectDir))
				{
					Variables["ProjectDir"] = ProjectDir;
				}
			}

			var oldWorkingDir = Environment.CurrentDirectory;

			string toolName = "unsync.exe";

			// If config specifies the explicit tool path, then use that.
			// Otherwise try same directory as config file.
			// Finally, try to look in the original working directory.

			string newWorkingDirTool = Path.GetFullPath(toolName);
			string oldWorkingDirTool = Path.GetFullPath(Path.Combine(oldWorkingDir, toolName));

			if (Config != null && Config.UnsyncPath != null)
			{
				UnsyncPath = Config.UnsyncPath;
			}
			else if (File.Exists(newWorkingDirTool))
			{
				UnsyncPath = newWorkingDirTool;
			}
			else if (File.Exists(oldWorkingDirTool))
			{
				UnsyncPath = oldWorkingDirTool;
			}

			if (File.Exists(configFile))
			{
				// Move the working dir into the same as the config file.
				// This allows relative paths to automatically be relative to the config.
				var workingDir = Path.GetDirectoryName(Path.GetFullPath(configFile));
				Environment.CurrentDirectory = workingDir;

				try
				{
					Config = new Config(configFile, UnsyncPath, Variables);
				}
				catch (Exception ex)
				{
					MessageBox.Show($"Failed to load configuration from \"{configFile}\".\n\n{ex}", "Fatal error");
					Shutdown(1);
				}
			}
			else if (e.Args.Length > 0)
			{
				MessageBox.Show($"The configuration file \"{configFile}\" does not exist. Projects will not be available.");
			}

			UserConfig = UserPreferences.Load();

			if (Config != null)
			{
				/*
				if (Config.RootProxy != null)
				{
					App.Current.LogMessage($"Unsync server address: {Config.RootProxy.Path}");
					EnableUserAuthentication = true;
				}
				else
				{
					App.Current.LogMessage($"Unsync server address is not configured");
					EnableUserAuthentication = false;
				}
				*/

				EnableUserAuthentication = true;

				Config.UnsyncPath = UnsyncPath;
				Config.EnableExperimentalFeatures = EnableExperimentalFeatures;
				Config.EnableUserAuthentication = EnableUserAuthentication;
			}

			if (UnsyncPath != null)
			{
				try
				{
					using (var proc = new System.Diagnostics.Process())
					{
						proc.StartInfo = new System.Diagnostics.ProcessStartInfo(UnsyncPath, "sync --help")
						{
							CreateNoWindow = true,
							UseShellExecute = false,
							RedirectStandardOutput = true,
							RedirectStandardError = true
						};
						proc.Start();
						string helpText = proc.StandardOutput.ReadToEnd();
						if (proc.WaitForExit(5000))
						{
							UnsyncSupportsStdout = helpText.Contains("--stdout");
						}
						else
						{
							proc.Kill();
						}
					}
				}
				catch { }
			}

			base.OnStartup(e);
		}

		protected override void OnExit(ExitEventArgs e)
		{
			if (UserConfig != null)
			{
				UserConfig.Save();
			}
			base.OnExit(e);
		}

		static void UnhandledException(object sender, string e)
		{
			MessageBox.Show($"Unhandled exception:\n{e}", "Fatal error", MessageBoxButton.OK, MessageBoxImage.Error);
			Environment.Exit(1);
		}
	}

	public sealed class UserPreferences
	{
		// Change this guid if any changes are made which break compat with existing user preferences files
		private static readonly Guid version = Guid.Parse("1386020B-374C-43CE-8FBF-74AAE69EEFD3");
		private static readonly string userDir = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "UnsyncUI");
		private static readonly string userFile = Path.Combine(userDir, $"user_{version}.json");

		public string Proxy { get; set; }
		public Dictionary<string, string> ProjectDestinationMap { get; set; } = new Dictionary<string, string>();
		public string CustomSrcPath { get; set; }
		public string CustomDstPath { get; set; }
		public string CustomInclude { get; set; }
		public string AdditionalArgs { get; set; }
		public bool AppendBuildName { get; set; } = false;
		public bool LogInOnStartup { get; set; } = false;

		public static UserPreferences Load()
		{
			try
			{
				if (File.Exists(userFile))
				{
					return JsonSerializer.Deserialize<UserPreferences>(File.ReadAllText(userFile));
				}
			}
			catch (Exception ex)
			{
				MessageBox.Show($"Failed to load user preferences from \"{userFile}\". {ex}");
			}

			return new UserPreferences();
		}

		public void Save()
		{
			try
			{
				if (!Directory.Exists(userDir))
					Directory.CreateDirectory(userDir);

				File.WriteAllText(userFile, JsonSerializer.Serialize(this, new JsonSerializerOptions() { WriteIndented = true }));
			}
			catch { }
		}
	}
}
