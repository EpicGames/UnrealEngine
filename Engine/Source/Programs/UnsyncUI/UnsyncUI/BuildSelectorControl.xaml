<!--Copyright Epic Games, Inc. All Rights Reserved.-->

<UserControl x:Class="UnsyncUI.BuildSelectorControl"
             xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006" 
             xmlns:d="http://schemas.microsoft.com/expression/blend/2008" 
             xmlns:local="clr-namespace:UnsyncUI"
			 xmlns:dat="clr-namespace:System.ComponentModel;assembly=WindowsBase"
             mc:Ignorable="d" 
             d:DesignHeight="450" d:DesignWidth="800">

	<Grid>
		<Grid.RowDefinitions>
			<RowDefinition Height="*" />
			<RowDefinition Height="auto" />
			<RowDefinition Height="auto" />
			<RowDefinition Height="auto" />
		</Grid.RowDefinitions>

		<!-- Builds Groupbox -->
		<GroupBox Grid.Row="0">

			<GroupBox.Header>
				<Grid>
					<TextBlock Margin="4" Visibility="{Binding Path=OnRefreshBuildsClicked.Enabled, Converter={StaticResource VisConv_Collapsed}}">
					<Run Text="Builds -"/>
					<Hyperlink Command="{Binding OnRefreshBuildsClicked}">Refresh</Hyperlink>
					</TextBlock>

					<TextBlock Margin="4" Visibility="{Binding Path=OnStopRefreshBuildsClicked.Enabled, Converter={StaticResource VisConv_Collapsed}}">
					<Run Text="Builds -"/>
					<Hyperlink Command="{Binding OnStopRefreshBuildsClicked}">Stop</Hyperlink>
					</TextBlock>
				</Grid>
			</GroupBox.Header>

			<Grid>
				<Grid.RowDefinitions>
					<RowDefinition Height="auto"/>
					<RowDefinition Height="auto"/>
					<RowDefinition Height="*"/>
				</Grid.RowDefinitions>

				<ProgressBar Grid.Row="0" IsIndeterminate="True" Visibility="{Binding Path=OnStopRefreshBuildsClicked.Enabled, Converter={StaticResource VisConv_Hidden}}" Margin="4,2,4,2"/>

				<Grid Grid.Row="1">
					<Grid.ColumnDefinitions>
						<ColumnDefinition Width="*"/>
						<ColumnDefinition Width="auto"/>
					</Grid.ColumnDefinitions>
					<TextBox Grid.Row ="0" Grid.Column="0" Text="{Binding SearchTerms, UpdateSourceTrigger=PropertyChanged}" />
					<TextBlock Grid.Row="0" Grid.Column="1" Text="Search" Margin="8,4,8,4"/>
				</Grid>

				<ListView Grid.Row="2" SelectionMode="Single" ItemsSource="{Binding Builds}" SelectedItem="{Binding SelectedBuild}" BorderBrush="Transparent" Name="SelectedBuild" GridViewColumnHeader.Click="BuildsHeader_Click">
					<ListView.ContextMenu>
						<ContextMenu>
							<MenuItem Header="Copy Build CL" Command="{Binding CopyBuildCLClicked}" />
							<MenuItem Header="Copy Build Name" Command="{Binding CopyBuildNameClicked}" />
							<MenuItem Header="Open Build Location" Command="{Binding OpenBuildLocationClicked}" />
						</ContextMenu>
					</ListView.ContextMenu>
					<ListView.View>
						<GridView>
							<GridViewColumn x:Name="colStream" Width="200" Header="Stream" DisplayMemberBinding="{Binding Stream}" />
							<GridViewColumn x:Name="colCL" Width="100" Header="CL" DisplayMemberBinding="{Binding CL}" />
							<GridViewColumn x:Name="colSuffix" Width="100" Header="Suffix" DisplayMemberBinding="{Binding Suffix}" />
							<GridViewColumn x:Name="colTimestamp" Width="125" Header="Timestamp" DisplayMemberBinding="{Binding Timestamp}" />
						</GridView>
					</ListView.View>
				</ListView>
			</Grid>
		</GroupBox>

		<TextBlock TextAlignment="Center" VerticalAlignment="Center" Visibility="{Binding Path=StatusString, Converter={StaticResource VisConv_Collapsed}}">
			<Run Text="{Binding Path=StatusString, Mode=OneWay}"/>
		</TextBlock>

		<!-- Platforms Groupbox -->
		<GroupBox Grid.Row="1" MinHeight="62" DataContext="{Binding SelectedItem, ElementName=SelectedBuild}" Visibility="{Binding SelectedItem, ElementName=SelectedBuild, Converter={StaticResource VisConv_Collapsed}}">
			
			<GroupBox.Header>
				<Grid>
					<TextBlock Margin="4" Visibility="{Binding Path=OnRefreshPlatformsClicked.Enabled, Converter={StaticResource VisConv_Collapsed}}">
					<Run Text="Platforms -"/>
					<Hyperlink Command="{Binding OnRefreshPlatformsClicked}">Refresh</Hyperlink>
					</TextBlock>

					<TextBlock Margin="4" Visibility="{Binding Path=OnStopRefreshPlatformsClicked.Enabled, Converter={StaticResource VisConv_Collapsed}}">
					<Run Text="Platforms -"/>
					<Hyperlink Command="{Binding OnStopRefreshPlatformsClicked}">Stop</Hyperlink>
					</TextBlock>
				</Grid>
			</GroupBox.Header>
			
			<Grid>
				<Grid.RowDefinitions>
					<RowDefinition Height="auto"/>
					<RowDefinition Height="*"/>
				</Grid.RowDefinitions>

				<ProgressBar Grid.Row="0" IsIndeterminate="True" Visibility="{Binding Path=OnStopRefreshPlatformsClicked.Enabled, Converter={StaticResource VisConv_Hidden}}" Margin="4,2,4,2"/>

				<TextBlock Grid.Row="0" Grid.RowSpan="2" Text="No platforms found for this build" VerticalAlignment="Center" HorizontalAlignment="Center">
					<TextBlock.Visibility>
						<MultiBinding Converter="{StaticResource VisConv_All_Collapsed}">
							<Binding Path="Platforms.Count" Converter="{StaticResource NotConv}"/>
							<Binding Path="OnRefreshPlatformsClicked.Enabled"/>
						</MultiBinding>
					</TextBlock.Visibility>
				</TextBlock>

				<ListView Grid.Row="1" Foreground="Transparent" Background="Transparent" BorderBrush="Transparent" ItemsSource="{Binding Platforms}" Name="SelectedPlatforms">
					<ListView.ItemTemplate>
						<DataTemplate>
							<Grid>
								<Grid.ColumnDefinitions>
									<ColumnDefinition SharedSizeGroup="WrapPanelGroup" />
								</Grid.ColumnDefinitions>
								<CheckBox Margin="4,0,4,0" Content="{Binding Description}" IsChecked="{Binding Path=IsSelected, RelativeSource={RelativeSource FindAncestor, AncestorType={x:Type ListViewItem}}}"/>
							</Grid>
						</DataTemplate>
					</ListView.ItemTemplate>
					<ListView.ItemsPanel>
						<ItemsPanelTemplate>
							<WrapPanel Orientation="Horizontal" Grid.IsSharedSizeScope="True"
									   Width="{Binding RelativeSource={RelativeSource FindAncestor,AncestorType={x:Type ScrollViewer}},Path=ActualWidth}" />
						</ItemsPanelTemplate>
					</ListView.ItemsPanel>
					<ListView.ItemContainerStyle>
						<Style TargetType="ListViewItem">
							<Setter Property="Template">
								<Setter.Value>
									<ControlTemplate TargetType="ListViewItem">
										<ContentPresenter/>
									</ControlTemplate>
								</Setter.Value>
							</Setter>
						</Style>
					</ListView.ItemContainerStyle>
				</ListView>
			</Grid>
			
		</GroupBox>

		<!-- Destination Path -->
		<GroupBox Grid.Row="2">
			<GroupBox.Header>
				<Grid>
					<TextBlock Margin="4" Visibility="{Binding Path=ResetDestinationClicked.Enabled, Converter={StaticResource VisConv_Collapsed_Inv}}">
						<Run Text="Destination"/>
					</TextBlock>
					<TextBlock Margin="4" Visibility="{Binding Path=ResetDestinationClicked.Enabled, Converter={StaticResource VisConv_Collapsed}}">
						<Run Text="Destination - "/>
						<Hyperlink Command="{Binding ResetDestinationClicked}">Reset</Hyperlink>
					</TextBlock>
				</Grid>
			</GroupBox.Header>

			<Grid>
				<Grid.RowDefinitions>
					<RowDefinition Height="auto"/>
					<RowDefinition Height="auto"/>
					<RowDefinition Height="auto"/>
				</Grid.RowDefinitions>

				<local:SelectFolderControl Grid.Row ="0" SelectedPath="{Binding DstPath, UpdateSourceTrigger=PropertyChanged}" Description="Select a destination directory to sync builds to."/>

				<CheckBox Grid.Row ="1" VerticalAlignment="Center" IsChecked="{Binding AppendBuildName}" IsEnabled="{Binding SelectedBuild, Converter={StaticResource IsObjectNotNull}}"
						  Content="Append Build Name" ToolTip="If a build is selected, its name will be appended to the destination folder.&#x0a;Note : using this option will download each build in a different folder, which is useful when you need multiple builds for comparison or whatnot. However, downloading builds over an existing folder yields faster sync times." 
						  ToolTipService.ShowOnDisabled="True" HorizontalAlignment="Left" Margin="4,0,0,0"/>
				<TextBlock Margin="4" Grid.Row ="2">
					<Hyperlink Command="{Binding OpenDestinationClicked}">
						<TextBlock Text="{Binding FinalDstPath, UpdateSourceTrigger=PropertyChanged, Mode=OneWay}"/>
					</Hyperlink>
				</TextBlock>
			</Grid>
		</GroupBox>

		<!-- Start sync -->
		<Grid Grid.Row="3">
			<CheckBox Content="Dry Run" IsChecked="{Binding DryRun}" ToolTip="Download and verify the data, but don't write outputs to disk" Margin="10,4,4,4" HorizontalAlignment="Left"/>
			<TextBlock Margin="4" HorizontalAlignment="Center">
				<Hyperlink Command="{Binding OnStartSync}" CommandParameter="{Binding SelectedItems, ElementName=SelectedPlatforms}">
					<Hyperlink.IsEnabled>
						<MultiBinding Converter="{StaticResource BoolConv_All}">
							<Binding ElementName="SelectedPlatforms" Path="SelectedItems.Count"/>
							<Binding Path="DstPath.Length"/>
						</MultiBinding>
					</Hyperlink.IsEnabled>
					Start Sync
				</Hyperlink>
			</TextBlock>
		</Grid>
	</Grid>
</UserControl>
