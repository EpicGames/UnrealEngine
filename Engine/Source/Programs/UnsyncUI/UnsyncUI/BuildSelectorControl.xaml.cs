// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.ComponentModel;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Threading.Tasks.Dataflow;
using System.Windows;
using System.Windows.Controls;
using System.Windows.Data;

namespace UnsyncUI
{
	/// <summary>
	/// Interaction logic for BuildSelectorControl.xaml
	/// </summary>
	public partial class BuildSelectorControl : UserControl
	{
		GridViewColumnHeader lastHeaderClicked = null;
		ListSortDirection lastDirection = ListSortDirection.Ascending;

		public BuildSelectorControl()
		{
			InitializeComponent();
		}

		private void SyncHyperlink_Click(object sender, RoutedEventArgs e)
		{
			(DataContext as ProjectModel).StartSync(SelectedPlatforms.SelectedItems.OfType<BuildPlatformModel>());
		}

		private void BuildsHeader_Click(object sender, RoutedEventArgs e)
		{
			var headerClicked = e.OriginalSource as GridViewColumnHeader;
			if (headerClicked == null || headerClicked.Role == GridViewColumnHeaderRole.Padding)
				return;

			ListSortDirection direction;
			if (headerClicked != lastHeaderClicked)
			{
				direction = ListSortDirection.Ascending;
			}
			else
			{
				direction = lastDirection == ListSortDirection.Ascending
					? ListSortDirection.Descending
					: ListSortDirection.Ascending;
			}

			if (headerClicked.Column == colStream)
			{
				Sort((colStream.Header as string, direction), (colCL.Header as string, ListSortDirection.Descending));
			}
			else
			{
				var columnBinding = headerClicked.Column.DisplayMemberBinding as Binding;
				var sortBy = columnBinding?.Path.Path ?? headerClicked.Column.Header as string;

				Sort((sortBy, direction));
			}

			lastHeaderClicked = headerClicked;
			lastDirection = direction;
		}

		private void Sort(params (string field, ListSortDirection direction)[] args)
		{
			var dataView = CollectionViewSource.GetDefaultView(SelectedBuild.ItemsSource);
			dataView.SortDescriptions.Clear();
			foreach (var desc in args)
			{
				dataView.SortDescriptions.Add(new SortDescription(desc.field, desc.direction));
			}
			dataView.Refresh();
		}
	}

	public sealed class BuildPlatformModel
	{

		public BuildModel Build { get; }
		public string Platform { get; }
		public string Flavor { get; }
		public string Description { get; }
		public string FullPath { get; }
		public string DestPathRelative { get; }
		public string Include { get;  }

		public string Name => Build == null
			? "Custom Sync" 
			: $"{Build.Stream} - {Platform} - CL {Build.CL}";

		public string SafeName => Name.Replace('/', '+').Replace(" ", "");

		public BuildPlatformModel(BuildModel build, string platform, string fullPath, string flavor, string include)
		{
			Build = build;
			Platform = platform;
			FullPath = fullPath;
			Flavor = flavor;
			DestPathRelative = Flavor == null ? Platform : Platform + Flavor;

			Description = Flavor == null ? Platform : $"{Platform} {Flavor}";
			Include = include;
		}
	}

	public sealed class BuildModel : BaseModel
	{
		private CancellationTokenSource cts;
		public string Stream => Template.Stream;
		public UInt64 CL => Template.CL == null ? 0 : (UInt64)Template.CL;
		public string Suffix => Template.Suffix == null ? "" : Template.Suffix;

		public string Timestamp => DirEntry.MTime == DateTime.MinValue ? "" : DirEntry.MTime.ToString("yyyy-MM-dd HH:mm:ss");

		public Config.Directory RootDir { get; }
		public string Path => DirEntry.Path;

		internal Config.BuildTemplate Template { get; }

		public Command OnRefreshPlatformsClicked { get; }
		public Command OnStopRefreshPlatformsClicked { get; }

		public ObservableCollection<BuildPlatformModel> Platforms { get; } = new ObservableCollection<BuildPlatformModel>();

		private Config AppConfig;

		private DirectoryEntry DirEntry;

		internal BuildModel(DirectoryEntry dirEntry, Config.Directory rootDir, Config.BuildTemplate template, Config config)
		{
			AppConfig = config;
			DirEntry = dirEntry;
			RootDir = rootDir;
			Template = template;

			OnRefreshPlatformsClicked = new Command(() => PopulatePlatforms()) { Enabled = true };
			OnStopRefreshPlatformsClicked = new Command(() =>
			{
				cts?.Cancel();
			});
		}

		public bool HasPlatform(string platform) => Platforms.Where(p => p.Platform == platform).Any();

		private async Task EnumeratePlatforms(IDirectoryEnumerator dirEnum, Config.Directory currentDir, string path, Config.BuildTemplate template, CancellationTokenSource cancellationToken)
		{
			if (!currentDir.Parse(path, ref template))
				return;

			try
			{
				foreach (var childDir in await dirEnum.EnumerateDirectories(path, cancellationToken.Token))
				{
					if (System.IO.Path.GetFileName(childDir.Path) == ".unsync")
					{
						List<Config.BuildTemplate> fileGroups = null;
						if (currentDir.FileGroups?.Any() == true)
						{
							var dirEntries = await dirEnum.EnumerateFiles(path, cancellationToken.Token);
							var files = DirectoryEntry.ToPaths(dirEntries);
							currentDir.ParseFileGroups(files.ToList(), template, out fileGroups);
						}

						string formattedPath = dirEnum.FormatArtifactPath(path);

						if (fileGroups?.Any() == true)
						{
							foreach (var fileGroup in fileGroups)
							{
								Platforms.Add(new BuildPlatformModel(this, fileGroup.Platform, formattedPath, fileGroup.Flavor, fileGroup.Include));
							}
							Platforms.Add(new BuildPlatformModel(this, template.Platform, formattedPath, "All", null));
						}
						else
						{
							// This folder is a valid build
							Platforms.Add(new BuildPlatformModel(this, template.Platform, formattedPath, template.Flavor, null));
						}
					}
					else
					{
						foreach (var subDir in currentDir.SubDirectories)
						{
							await EnumeratePlatforms(dirEnum, subDir, childDir.Path, template, cancellationToken);
						}
					}
				}
			}
			catch (Exception ex)
			{
				App.Current?.LogMessage($"Platforms enumeration failed with exception: {ex}");
			}
		}

		public async void PopulatePlatforms()
		{
			cts?.Cancel();
			cts = new CancellationTokenSource();

			OnRefreshPlatformsClicked.Enabled = false;
			OnStopRefreshPlatformsClicked.Enabled = true;

			Platforms.Clear();

			IDirectoryEnumerator dirEnum = AppConfig.CreateDirectoryEnumerator(Path, RootDir);

			try
			{
				await EnumeratePlatforms(dirEnum, RootDir, Path, Template, cts);
			}
			catch (OperationCanceledException)
			{ }
			finally
			{
				OnRefreshPlatformsClicked.Enabled = true;
				OnStopRefreshPlatformsClicked.Enabled = false;
			}
		}
	}

	public sealed class ProjectModel : TabModel
	{
		private bool firstRefresh = true;
		private Action<IEnumerable<(SyncStartConfig Config, BuildPlatformModel Model)>> onBuildsSelected;

		private string statusString = null;
		public string StatusString
		{
			get => statusString;
			set
			{
				SetProperty(ref statusString, value);
			}
		}

		public Config.Project Definition { get; }
		public string Name => Definition.Name;

		public Command OnRefreshBuildsClicked { get; }
		public Command OnStopRefreshBuildsClicked { get; }
		public Command<IList> OnStartSync { get; }
		public Command CopyBuildCLClicked { get; }
		public Command CopyBuildNameClicked { get; }
		public Command OpenBuildLocationClicked { get; }
		public Command ResetDestinationClicked { get; }
		public Command OpenDestinationClicked { get; }

		public ObservableCollection<BuildModel> Builds { get; } = new ObservableCollection<BuildModel>();
		public ICollectionView BuildsView { get; }

		public bool BuildsViewFilter(object item)
		{
			BuildModel build = (BuildModel)item;

			foreach (var term in searchTerms.Split(' '))
			{
				if (!build.CL.ToString().Contains(term, StringComparison.InvariantCultureIgnoreCase)
				&& !build.Stream.Contains(term, StringComparison.InvariantCultureIgnoreCase)
				&& !build.Suffix.Contains(term, StringComparison.InvariantCultureIgnoreCase))
				{
					return false;
				}
			}

			return true;
		}

		private BuildModel selectedBuild = null;
		public BuildModel SelectedBuild
		{
			get => selectedBuild;
			set
			{
				SetProperty(ref selectedBuild, value);

				UpdateFinalDstPath();

				if (selectedBuild != null)
				{
					selectedBuild.PopulatePlatforms();
				}
			}
		}

		private string searchTerms = "";
		public string SearchTerms
		{
			get => searchTerms;
			set
			{
				SetProperty(ref searchTerms, value);
				BuildsView.Refresh();
			}
		}

		private CancellationTokenSource cts = null;

		/// Default download directory (either auto-detected or loaded from project config)
		private string defaultDstPath = null;

		/// Current download path selected by user or loaded from user preferences.
		/// This value persists between application sessions (saved per project).
		/// It can be reset to default by pressing "Reset" button in the UI.
		private string dstPath = null;

		public string DstPath
		{
			get => dstPath;
			set
			{
				SetProperty(ref dstPath, value);

				UpdateFinalDstPath();

				App.Current.UserConfig.ProjectDestinationMap[Name] = DstPath;
			}
		}

		void UpdateFinalDstPath()
		{
			string finalPath = dstPath;
			if (AppendBuildName && (selectedBuild != null))
			{
				string buildFolderName = new DirectoryInfo(selectedBuild.Path).Name;
				finalPath = Path.Combine(DstPath, buildFolderName);
			}

			finalPath = Path.TrimEndingDirectorySeparator(finalPath);
			if (FinalDstPath != finalPath)
			{
				FinalDstPath = finalPath;
			}

			ResetDestinationClicked.Enabled = CanResetDestinationToDefault;
		}

		private string finalDstPath = null;
		public string FinalDstPath
		{
			get => finalDstPath;
			set
			{
				SetProperty(ref finalDstPath, value);
			}
		}

		public bool DryRun { get; set; }

		private bool appendBuildName = true;
		public bool AppendBuildName
		{
			get => appendBuildName;
			set
			{
				SetProperty(ref appendBuildName, value);

				App.Current.UserConfig.AppendBuildName = value;

				UpdateFinalDstPath();
			}
		}

		public ProjectModel(Config.Project definition, Action<IEnumerable<(SyncStartConfig Config, BuildPlatformModel Model)>> onBuildsSelected)
		{
			this.Definition = definition;
			this.onBuildsSelected = onBuildsSelected;
			OnRefreshBuildsClicked = new Command(RefreshBuildsAsync) { Enabled = true };
			OnStopRefreshBuildsClicked = new Command(StopRefreshBuilds) { Enabled = false };
			OnStartSync = new Command<IList>(l => StartSync(l.OfType<BuildPlatformModel>())) { Enabled = true };
			CopyBuildCLClicked = new Command(CopyBuildCL) { Enabled = true };
			CopyBuildNameClicked = new Command(CopyBuildName) { Enabled = true };
			OpenBuildLocationClicked = new Command(OpenBuildLocation) { Enabled = true };
			OpenDestinationClicked = new Command(OpenDestinationLocation) { Enabled = true };

			BuildsView = CollectionViewSource.GetDefaultView(Builds);
			BuildsView.Filter = BuildsViewFilter;
			BuildsView.SortDescriptions.Clear();
			BuildsView.SortDescriptions.Add(new SortDescription("CL", ListSortDirection.Descending));

			SearchTerms = App.Current.DefaultSearchTerms;

			if (definition.Destination != null)
			{
				defaultDstPath = Path.GetFullPath(definition.Destination);
			}
			else
			{
				// If this is the first time running, choose the destination dir as the largest local disk
				// @todo: remember user choice
				var bestDrive = DriveInfo.GetDrives()
				.Where(d => d.DriveType == DriveType.Fixed)
				.Where(d => !d.VolumeLabel.Contains("google", StringComparison.InvariantCultureIgnoreCase)) // Ignore Google Drive (it appears as a 'fixed' disk)
				.OrderByDescending(d => d.TotalSize).FirstOrDefault();
				if (bestDrive != null)
				{
					defaultDstPath = Path.Combine(bestDrive.RootDirectory.FullName, "Unsync", definition.Name);
				}
			}

			// If we failed to find a reasonable default destination then set it to a non-null empty string,
			// which prevents sync command until user explicitly sets the path
			if (defaultDstPath == null)
			{
				defaultDstPath = "";
			}

			appendBuildName = App.Current.UserConfig.AppendBuildName;

			ResetDestinationClicked = new Command(ResetDestinationToDefault) { Enabled = CanResetDestinationToDefault };

			string userDstPath = null;

			if (App.Current.UserConfig.ProjectDestinationMap.TryGetValue(Name, out userDstPath))
			{
				dstPath = userDstPath;
			}
			else
			{
				dstPath = defaultDstPath;
			}
			UpdateFinalDstPath();
		}
		private bool CanResetDestinationToDefault => finalDstPath != defaultDstPath && defaultDstPath.Length != 0;

		private void ResetDestinationToDefault()
		{
			DstPath = defaultDstPath;
			AppendBuildName = false;
		}

		private void StopRefreshBuilds()
		{
			if (cts != null)
			{
				cts.Cancel();
				OnStopRefreshBuildsClicked.Enabled = false;
			}
		}

		public async void RefreshBuildsAsync()
		{
			cts = new CancellationTokenSource();

			OnRefreshBuildsClicked.Enabled = false;
			OnStopRefreshBuildsClicked.Enabled = true;

			Builds.Clear();
			SelectedBuild = null;
			StatusString = null;

			try
			{
				var timer = new Stopwatch();
				timer.Start();
				var (task, buildsPipe) = Definition.EnumerateBuilds(cts.Token);

				while (await buildsPipe.OutputAvailableAsync(cts.Token))
				{
					Builds.Add(await buildsPipe.ReceiveAsync());
				}

				timer.Stop();

				if (task.IsFaulted)
				{
					throw task.Exception.InnerException;
				}
				else
				{
					App.Current.LogMessage($"Search took {timer.Elapsed.TotalSeconds} seconds.");
				}
			}
			catch (OperationCanceledException)
			{ }
			catch (Exception ex)
			{
				App.Current.LogMessage($"Directory enumeration failed with exception: {ex}");
				StatusString = ex.Message;
			}
			finally
			{
				OnRefreshBuildsClicked.Enabled = true;
				OnStopRefreshBuildsClicked.Enabled = false;
			}
		}

		public void StartSync(IEnumerable<BuildPlatformModel> selectedBuilds)
		{
			onBuildsSelected?.Invoke(selectedBuilds.Select(s => {
				var Config = new SyncStartConfig();
				Config.DstPath = Path.Combine(finalDstPath, s.DestPathRelative);
				Config.Exclusions = Definition.Exclusions?.ToArray();
				Config.DryRun = DryRun;
				if (App.Current.UserConfig.AppendBuildName)
				{
					Config.ScavengePath = DstPath;
				}
				return (Config, s);
			}).ToList());
		}

		public override void OnSelected()
		{
			if (firstRefresh)
			{
				RefreshBuildsAsync();
				firstRefresh = false;
			}
		}

		private void CopyStringToClipboard(string str)
		{
			// Clipboard can be locked if timing is just right(wrong), retry a few times on failure.
			int retryDelay = 0;
			const int attempts = 7;
			for (int attempt = 0; attempt < attempts; attempt++)
			{
				if (retryDelay > 0)
				{
					System.Threading.Thread.Sleep(retryDelay);
					retryDelay *= 2;
				}
				else
				{
					retryDelay = 5;
				}
				try
				{
					Clipboard.SetText(str, TextDataFormat.Text);
					break;
				}
				catch (System.Runtime.InteropServices.COMException)
				{
				}
			}
		}

		private void CopyBuildCL()
		{
			if (selectedBuild != null)
			{
				CopyStringToClipboard(selectedBuild.CL.ToString());
			}
		}

		private void CopyBuildName()
		{
			if (selectedBuild != null)
			{
				string buildFolderName = new DirectoryInfo(selectedBuild.Path).Name;
				CopyStringToClipboard(buildFolderName);
			}
		}

		private void OpenBuildLocation()
		{
			if (selectedBuild != null)
			{
				Shell.LaunchExplorer(selectedBuild.Path);
			}
		}

		private void OpenDestinationLocation()
		{
			if (!System.IO.Directory.Exists(FinalDstPath))
			{
				try
				{
					System.IO.Directory.CreateDirectory(FinalDstPath);
				}
				catch (Exception)
				{
				}
			}
			Shell.LaunchExplorer(FinalDstPath);
		}
	}
}
