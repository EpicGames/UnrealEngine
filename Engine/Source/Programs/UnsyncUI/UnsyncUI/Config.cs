// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using System.Threading.Tasks.Dataflow;
using System.Xml.Linq;
using System.Net;
using System.Text.Json;

namespace UnsyncUI
{
	public sealed class Config
	{
		public sealed class Proxy
		{
			public string Name { get; set; }
			public string Path { get; set; }
			public string Protocol { get; set; }

			public UnsyncServerConfig GetServerConfig()
			{
				return new UnsyncServerConfig { address=Path, protocol=Protocol };
			}

		}

		internal struct BuildTemplate
		{
			public string Stream;
			public UInt64? CL;
			public string Suffix;
			public string Platform;
			public string Flavor;
			public string Include;

			public bool IsStreamClFound => Stream != null && CL != null;

		}

		public sealed class FileGroup
		{
			public Regex Regex { get; }
			public string Include { get; }
			public string Flavor { get; }

			public FileGroup(XElement node)
			{
				Regex = new Regex($@"^{node.Attribute("regex")?.Value}$", RegexOptions.IgnoreCase);
				Include = node.Attribute("include")?.Value;
				Flavor = node.Attribute("flavor")?.Value;
			}
		}

		public sealed class Directory
		{
			public Regex Regex { get; }
			public string Stream { get; }
			public string Suffix { get; }
			public string CL { get; }
			public string Platform { get; }
			public string Flavor { get; }
			public List<Directory> SubDirectories { get; }
			public List<FileGroup> FileGroups { get; }

			public Directory(XElement node)
			{
				Regex = new Regex($@"^{node.Attribute("regex")?.Value}$", RegexOptions.IgnoreCase);
				Stream = node.Attribute("stream")?.Value;
				Suffix = node.Attribute("suffix")?.Value;
				CL = node.Attribute("cl")?.Value;
				Platform = node.Attribute("platform")?.Value;
				Flavor = node.Attribute("flavor")?.Value;
				SubDirectories = node.Elements("dir").Select(d => new Directory(d)).ToList();
				FileGroups = node.Elements("files").Select(f => new FileGroup(f)).ToList();
			}

			internal bool Parse(string path, ref BuildTemplate template)
			{
				var match = Regex.Match(Path.GetFileName(path));
				if (!match.Success)
				{
					return false;
				}

				string RegexReplace(string field) => Regex.Replace(field, @"\$([0-9]+)", m => match.Groups[int.Parse(m.Groups[1].Value)].Value);

				if (Stream != null)
				{
					template.Stream = RegexReplace(Stream);
				}
				if (CL != null)
				{
					string CLString = RegexReplace(CL);
					UInt64 CLNum = 0;
					if (UInt64.TryParse(CLString, out CLNum))
					{
						template.CL = CLNum;
					}
				}
				if (Suffix != null)
				{
					template.Suffix = RegexReplace(Suffix);
				}
				if (Platform != null)
				{
					template.Platform = RegexReplace(Platform);
				}
				if (Flavor != null)
				{
					template.Flavor = RegexReplace(Flavor);
				}

				return true;
			}

			internal void ParseFileGroups(List<String> filePaths, BuildTemplate template, out List<BuildTemplate> groupTemplates)
			{
				groupTemplates = null;
				if (!filePaths.Any())
				{
					return;
				}

				HashSet<BuildTemplate> newTemplates = new HashSet<BuildTemplate>();
				foreach (string path in filePaths)
				{
					foreach (var fileGroup in FileGroups)
					{
						var match = fileGroup.Regex.Match(Path.GetFileName(path));
						if (match?.Success == true)
						{
							string RegexReplace(string field) => Regex.Replace(field, @"\$([0-9]+)", m => match.Groups[int.Parse(m.Groups[1].Value)].Value);

							BuildTemplate groupTemplate = template;
							if (fileGroup.Flavor != null)
							{
								groupTemplate.Flavor = RegexReplace(fileGroup.Flavor);
							}
							if (fileGroup.Include != null)
							{
								groupTemplate.Include = RegexReplace(fileGroup.Include);
							}
							newTemplates.Add(groupTemplate);
						}
					}
				}

				groupTemplates = newTemplates.ToList();
			}
		}

		public sealed class Project
		{
			internal Config AppConfig;

			public string Name { get; set; }
			public string Root { get; set; }
			public string Destination { get; set; }
			public List<string> Exclusions { get; set; }
			public List<Directory> Children { get; set; }

			private Task EnumerateBuilds(IDirectoryEnumerator dirEnum, ITargetBlock<BuildModel> pipe, Directory d, DirectoryEntry dirEntry, BuildTemplate template, CancellationToken cancelToken)
			{
				return Task.Run(async () =>
				{
					if (!d.Parse(dirEntry.Path, ref template))
						return;

					if (template.IsStreamClFound)
					{
						pipe.Post(new BuildModel(dirEntry, d, template, AppConfig));
					}
					else
					{
						var jobs = new List<Task>();
						foreach (var childDir in await dirEnum.EnumerateDirectories(dirEntry.Path, cancelToken))
						{
							jobs.AddRange(d.SubDirectories.Select(s => EnumerateBuilds(dirEnum, pipe, s, childDir, template, cancelToken)));
						}

						await Task.WhenAll(jobs);
					}
				});
			}

			public (Task, ISourceBlock<BuildModel>) EnumerateBuilds(CancellationToken cancelToken)
			{
				IDirectoryEnumerator dirEnum = AppConfig.CreateDirectoryEnumerator(this);

				var pipe = new BufferBlock<BuildModel>();
				var task = Task.Run(async () =>
				{
					try
					{
						var allTasks = new List<Task>();
						foreach (var dir in await dirEnum.EnumerateDirectories(Root, cancelToken))
						{
							allTasks.AddRange(Children.Select(c => EnumerateBuilds(dirEnum, pipe, c, dir, default(BuildTemplate), cancelToken)));
						}

						await Task.WhenAll(allTasks);
					}
					finally
					{
						pipe.Complete();
					}
				});

				return (task, pipe);
			}
		}

		public List<Proxy> Proxies { get; set; } = new List<Proxy>();
		public Proxy RootProxy { get; } = null;
		public List<Project> Projects { get; set; }

		public string UnsyncPath { get; set; }

		public bool EnableExperimentalFeatures { get; set; } = false;

		public bool EnableUserAuthentication { get; set; } = true;

		internal string loggedInUser;

		internal void ApplyVariables(XElement rootNode, Dictionary<string, string> Variables)
		{
			foreach (XElement node in rootNode.Elements())
			{
				foreach (XAttribute attrib in node.Attributes())
				{
					if (attrib.Value.Contains("$("))
					{
						foreach (KeyValuePair<string, string> KV in Variables)
						{
							string resolved = attrib.Value.Replace($"$({KV.Key})", KV.Value);
							attrib.SetValue(resolved);
						}
					}
				}

				ApplyVariables(node, Variables);
			}
		}

		public Config(string filename, string DefaultUnsyncPath = null, Dictionary<string, string> Variables = null)
		{
			var rootNode = XDocument.Load(filename).Root;

			if (Variables != null)
			{
				ApplyVariables(rootNode, Variables);
			}

			UnsyncPath = rootNode.Attribute("path")?.Value;
			if (UnsyncPath == null)
			{
				UnsyncPath = DefaultUnsyncPath;
			}

			if (UnsyncPath == null)
			{
				throw new Exception("Path to unsync.exe was not provided.");
			}

			if (!File.Exists(UnsyncPath))
			{
				throw new Exception("Unable to find unsync.exe binary specified in config file.");
			}

			List<Proxy> ConfigProxies = new List<Proxy>();

			var proxiesConfigNode = rootNode.Element("proxies");
			if (proxiesConfigNode != null)
			{
				ConfigProxies.AddRange(proxiesConfigNode.Elements("proxy").Select(p => new Proxy()
				{
					Name = p.Attribute("name")?.Value,
					Path = p.Attribute("path")?.Value,
					Protocol = p.Attribute("protocol")?.Value
				}));
			}

			// Set a default root proxy
			if (ConfigProxies.Count > 0)
			{
				RootProxy = ConfigProxies.First();
			}

			if (RootProxy != null)
			{
				// Horde requires a server connection, but unsync / native file system does not
				if (RootProxy.Protocol == "horde")
				{
					Proxies.AddRange(ConfigProxies);
				}
				else
				{
					Proxies.Add(new Proxy()
					{
						Name = "(none)",
						Path = null,
						Protocol = null,
					});

					// Auto-discover proxies
					(List<Proxy> DiscoveredProxies, Proxy DiscoveredRootProxy) = DiscoverProxies(ConfigProxies);

					if (DiscoveredRootProxy != null)
					{
						RootProxy = DiscoveredRootProxy;
					}

					if (DiscoveredProxies == null)
					{
						Proxies.AddRange(ConfigProxies);
					}
					else
					{
						Proxies.AddRange(DiscoveredProxies);
					}
				}
			}

			Projects = rootNode.Element("projects").Elements("project").Select(p => new Project()
			{
				AppConfig = this,
				Name = p.Attribute("name")?.Value,
				Root = p.Attribute("root")?.Value,
				Destination = p.Attribute("dest")?.Value,
				Children = p.Elements("dir").Select(d => new Directory(d)).ToList(),
				Exclusions = p.Elements("exclude").Select(d => d.Value).ToList()
			}).ToList();
		}

		// Returns list of mirrors and the seed proxy server that was used to get it or null
		private (List<Proxy>, Proxy) DiscoverProxies(List<Proxy> SeedServers)
		{
			foreach (Proxy SeedServer in SeedServers)
			{
				if (SeedServer.Path == null)
				{
					continue;
				}

				try
				{
					UnsyncQueryUtil QueryUtil = new UnsyncQueryUtil(UnsyncPath, SeedServer.GetServerConfig());

					var ParsedProxies = new List<Proxy>();

					foreach (var Mirror in QueryUtil.Mirrors())
					{
						if (!Mirror.ok || Mirror.ping == 0)
						{
							continue;
						}

						var ConvertedProxy = new Proxy();
						ConvertedProxy.Name = (Mirror.description != null && Mirror.description.Length > 0) ? Mirror.description : Mirror.name;
						ConvertedProxy.Path = Mirror.address;
						if (Mirror.port != 0)
						{
							ConvertedProxy.Path += ":" + Mirror.port.ToString();
						}
						ParsedProxies.Add(ConvertedProxy);
					}

					if (ParsedProxies.Count != 0)
					{
						return (ParsedProxies, SeedServer);
					}
				}
				catch (Exception ex)
				{
					App.Current.LogError("Exception during unsync query: " + ex.Message);
					continue;
				}
			}

			return (null, null);
		}

		private UnsyncQueryConfig CreateUnsyncQueryConfig()
		{
			UnsyncQueryConfig unsyncConfig = new UnsyncQueryConfig();
			unsyncConfig.server = RootProxy.GetServerConfig();
			unsyncConfig.unsyncPath = UnsyncPath;
			return unsyncConfig;
		}

		private bool CanUseUnsyncDirectoryEnumerator()
		{
			return EnableUserAuthentication
				&& RootProxy != null
				&& UnsyncPath != null
				&& loggedInUser != null;
		}

		public IDirectoryEnumerator CreateDirectoryEnumerator(Config.Project ProjectSchema)
		{
			if (CanUseUnsyncDirectoryEnumerator())
			{
				return new UnsyncDirectoryEnumerator(ProjectSchema, CreateUnsyncQueryConfig());
			}
			else
			{
				return new NativeDirectoryEnumerator();
			}
		}

		public IDirectoryEnumerator CreateDirectoryEnumerator(String Path, Config.Directory DirectorySchema)
		{
			if (CanUseUnsyncDirectoryEnumerator())
			{
				return new UnsyncDirectoryEnumerator(Path, DirectorySchema, CreateUnsyncQueryConfig());
			}
			else
			{
				return new NativeDirectoryEnumerator();
			}
		}

		public BuildModel CreateBuildModel(string path, DateTime timestamp, Config.Directory rootDir)
		{
			var dirEntry = new DirectoryEntry { Path = path, MTime = timestamp };
			return new BuildModel(dirEntry, rootDir, default(BuildTemplate), this);
		}

		public BuildModel CreateBuildModel(string path, Config.Directory rootDir)
		{
			return CreateBuildModel(path, DateTime.MinValue, rootDir);
		}

	} // class Config
}
