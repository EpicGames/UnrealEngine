// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Globalization;
using System.Linq;
using System.Windows;
using System.Windows.Data;
using System.Windows.Media;

namespace UnsyncUI
{
	public abstract class BaseConverter
	{
		protected static bool AsBool(object value)
		{
			if (value is bool)
			{
				return (bool)value;
			}
			else if (value is int)
			{
				return (int)value != 0;
			}
			else
			{
				//throw new ArgumentException("Unimplemented type.", nameof(value));
				return value != null;
			}
		}
	}
	public sealed class VisibilityConverter : IValueConverter
	{
		public Visibility TrueState { get; set; } = Visibility.Visible;
		public Visibility FalseState { get; set; } = Visibility.Collapsed;

		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
		{
			if (value is bool)
			{
				return (bool)value
					? TrueState
					: FalseState;
			}
			else if (value is int)
			{
				return (int)value != 0
					? TrueState
					: FalseState;
			}
			else if (value == null)
			{
				return FalseState;
			}
			else if (value is string)
			{
				return !string.IsNullOrEmpty((string)value)
					? TrueState
					: FalseState;
			}
			else
			{
				return TrueState;
			}
		}

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
			=> throw new NotImplementedException();
	}

	public sealed class NotConverter : BaseConverter, IValueConverter
	{
		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
			=> !AsBool(value);

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
			=> throw new NotImplementedException();
	}

	public sealed class AllConverter : BaseConverter, IMultiValueConverter
	{
		public object TrueValue { get; set; } = true;
		public object FalseValue { get; set; } = false;

		public object Convert(object[] values, Type targetType, object parameter, CultureInfo culture)
			=> values.All(o => AsBool(o)) ? TrueValue : FalseValue;

		public object[] ConvertBack(object value, Type[] targetTypes, object parameter, CultureInfo culture)
			=> throw new NotImplementedException();
	}

	public sealed class ObjectConverter : BaseConverter, IValueConverter
	{
		public object TrueValue { get; set; } = true;
		public object FalseValue { get; set; } = false;

		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
			=> AsBool(value) ? TrueValue : FalseValue;

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
			=> value == TrueValue;
	}

	public sealed class ConsoleColorConverter : IValueConverter
	{
		private Dictionary<ConsoleColor, SolidColorBrush> brushes = new Dictionary<ConsoleColor, SolidColorBrush>()
		{
			[ConsoleColor.Black] = new SolidColorBrush(Color.FromRgb(0, 0, 0)),
			[ConsoleColor.DarkBlue] = new SolidColorBrush(Color.FromRgb(0, 0, 128)),
			[ConsoleColor.DarkGreen] = new SolidColorBrush(Color.FromRgb(0, 128, 0)),
			[ConsoleColor.DarkCyan] = new SolidColorBrush(Color.FromRgb(0, 128, 128)),
			[ConsoleColor.DarkRed] = new SolidColorBrush(Color.FromRgb(128, 0, 0)),
			[ConsoleColor.DarkMagenta] = new SolidColorBrush(Color.FromRgb(128, 0, 128)),
			[ConsoleColor.DarkYellow] = new SolidColorBrush(Color.FromRgb(128, 128, 0)),
			[ConsoleColor.Gray] = new SolidColorBrush(Color.FromRgb(170, 170, 170)),
			[ConsoleColor.DarkGray] = new SolidColorBrush(Color.FromRgb(85, 85, 85)),
			[ConsoleColor.Blue] = new SolidColorBrush(Color.FromRgb(0, 0, 255)),
			[ConsoleColor.Green] = new SolidColorBrush(Color.FromRgb(0, 255, 0)),
			[ConsoleColor.Cyan] = new SolidColorBrush(Color.FromRgb(0, 255, 255)),
			[ConsoleColor.Red] = new SolidColorBrush(Color.FromRgb(255, 0, 0)),
			[ConsoleColor.Magenta] = new SolidColorBrush(Color.FromRgb(255, 0, 255)),
			[ConsoleColor.Yellow] = new SolidColorBrush(Color.FromRgb(255, 255, 0)),
			[ConsoleColor.White] = new SolidColorBrush(Color.FromRgb(255, 255, 255)),
		};

		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
		{
			if (brushes.TryGetValue((ConsoleColor)value, out var brush))
				return brush;

			return brushes[ConsoleColor.Black];
		}

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
			=> throw new NotImplementedException();
	}

	public class ObjectToBoolConverter : IValueConverter
	{
		public object Convert(object value, Type targetType, object parameter, CultureInfo culture)
		{
			bool returnValue = true;
			if (value == DependencyProperty.UnsetValue || value == null)
			{
				returnValue = false;
			}
			return returnValue;
		}

		public object ConvertBack(object value, Type targetType, object parameter, CultureInfo culture)
		{
			return Binding.DoNothing;
		}
	}
}
