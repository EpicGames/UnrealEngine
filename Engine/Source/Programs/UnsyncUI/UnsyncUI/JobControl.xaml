<!--Copyright Epic Games, Inc. All Rights Reserved.-->

<UserControl x:Class="UnsyncUI.JobControl"
             xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
             xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
             xmlns:local="clr-namespace:UnsyncUI"
             mc:Ignorable="d"
             d:DesignWidth="800">
	<UserControl.Resources>
		<local:ConsoleColorConverter x:Key="ConsoleColorConverter"/>
	</UserControl.Resources>
	<Grid>
		<Grid.RowDefinitions>
			<RowDefinition Height="auto"/>
			<RowDefinition Height="*"/>
		</Grid.RowDefinitions>
		<Grid Grid.Row="0">
			<Grid.ColumnDefinitions>
				<ColumnDefinition Width="auto"/>
				<ColumnDefinition Width="auto"/>
				<ColumnDefinition Width="*"/>
				<ColumnDefinition Width="100"/>
				<ColumnDefinition Width="auto"/>
			</Grid.ColumnDefinitions>
			<Grid.RowDefinitions>
				<RowDefinition/>
				<RowDefinition/>
			</Grid.RowDefinitions>
			<Expander  Grid.Column="0" VerticalAlignment="Center" Name="FilesExpander" IsExpanded="{Binding DetailsExpanded}" />
			<TextBlock Grid.Column="1" VerticalAlignment="Center" Text="{Binding Name}"/>
			<TextBlock Grid.Column="2" VerticalAlignment="Center" HorizontalAlignment="Right" Text="{Binding StatusMessage}" Foreground="{Binding StatusColor, Converter={StaticResource ConsoleColorConverter}}"/>
			<TextBlock Grid.Column="3" VerticalAlignment="Center" HorizontalAlignment="Right" Text="{Binding ProgressMessage}" Margin="4,4,8,4"/>
			<TextBlock Grid.Column="4" VerticalAlignment="Center" Margin="4">
                <TextBlock Visibility="{Binding ShowCancelHyperlink, Converter={StaticResource VisConv_Collapsed}}" >
                    <Hyperlink Command="{Binding OnCancelClicked}">Cancel</Hyperlink>
                </TextBlock>
                <TextBlock Visibility="{Binding ShowClearHyperlink, Converter={StaticResource VisConv_Collapsed}}" >
                    <Hyperlink Command="{Binding OnClearClicked}">Clear</Hyperlink>
                </TextBlock>
            </TextBlock>

			<ProgressBar Grid.Row="1" Grid.Column="0" Grid.ColumnSpan="5"
                         Value="{Binding TotalProgress, Mode=OneWay}"
                         IsIndeterminate="{Binding IsIndeterminate, Mode=OneWay}"
                         Visibility="{Binding ShowProgressBar, Converter={StaticResource VisConv_Collapsed}}" />
		</Grid>
		<Grid Grid.Row="1" Visibility="{Binding ElementName=FilesExpander, Path=IsExpanded, Converter={StaticResource VisConv_Collapsed}}">
			<Grid.ColumnDefinitions>
				<ColumnDefinition Width="auto"/>
				<ColumnDefinition Width="16"/>
				<ColumnDefinition Width="*"/>
			</Grid.ColumnDefinitions>
			<Grid.RowDefinitions>
				<RowDefinition Height="auto"/>
				<RowDefinition Height="auto"/>
				<RowDefinition Height="auto"/>
				<RowDefinition Height="*"/>
			</Grid.RowDefinitions>
			<TextBlock Grid.Column="0" Grid.Row="0" Text="Source:"/>
			<TextBlock Grid.Column="0" Grid.Row="1" Text="Destination:"/>
			<TextBlock Grid.Column="0" Grid.Row="2" Text="Server:"/>

			<TextBlock Grid.Column="2" Grid.Row="0">
                <Hyperlink Command="{Binding OnSourcePathClicked}">
                    <TextBlock Text="{Binding Build.FullPath, Mode=OneWay}"/>
                </Hyperlink>
            </TextBlock>

			<TextBlock Grid.Column="2" Grid.Row="1">
                <Hyperlink Command="{Binding OnDestPathClicked}">
                    <TextBlock Text="{Binding DstPath, Mode=OneWay}"/>
                </Hyperlink>
            </TextBlock>

			<TextBlock Grid.Column="2" Grid.Row="2" Text="{Binding ProxyStr}" />

			<GroupBox Grid.Row="3" Grid.ColumnSpan="3" Header="Log" Visibility="{Binding LogLines.Count, Converter={StaticResource VisConv_Collapsed}}">
				<Grid>
					<RichTextBox x:Name="LogRichTextBox"
								 IsReadOnly="True"
								 IsReadOnlyCaretVisible="True"
								 BorderThickness="0"
								 FontFamily="Consolas"
								 VerticalScrollBarVisibility="Auto"
								 HorizontalScrollBarVisibility="Disabled"
								 MaxHeight="{Binding MaxLogScrollViewHeight}"
								 Margin="0">
						<RichTextBox.ContextMenu>
							<ContextMenu>
								<MenuItem Header="Copy" Command="ApplicationCommands.Copy"/>
								<MenuItem Header="Copy All" Command="{Binding CopyLogClicked}"/>
								<MenuItem Header="Save As..." Command="{Binding SaveLogClicked}"/>
							</ContextMenu>
						</RichTextBox.ContextMenu>
						<RichTextBox.Resources>
							<Style TargetType="Paragraph">
								<Setter Property="Margin" Value="0"/>
							</Style>
						</RichTextBox.Resources>
						<FlowDocument PagePadding="0"/>
					</RichTextBox>
					<TextBlock x:Name="FollowLink" HorizontalAlignment="Right" VerticalAlignment="Bottom" Visibility="Collapsed" Margin="0,0,28,4">
						<Hyperlink Click="FollowLink_Click">Follow</Hyperlink>
					</TextBlock>
				</Grid>
			</GroupBox>
		</Grid>
	</Grid>
</UserControl>
