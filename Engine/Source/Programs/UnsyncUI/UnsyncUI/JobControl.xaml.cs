// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.Collections.Specialized;
using System.ComponentModel;
using System.Diagnostics;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using System.Windows;
using System.Windows.Controls;
using System.Windows.Documents;
using System.Windows.Media;

namespace UnsyncUI
{
	/// <summary>
	/// Interaction logic for ActiveJobControl.xaml
	/// </summary>
	public partial class JobControl : UserControl
    {
		private static readonly ConsoleColorConverter _colorConverter = new ConsoleColorConverter();

		private ScrollViewer _scrollViewer;
		private bool _autoScroll = true;
		private bool _isProgrammaticScroll = false;

		private FlowDocument _logDocument;
		private LogModel _currentLogModel;
		private Run _currentRun;

        public JobControl()
        {
            InitializeComponent();

			_logDocument = LogRichTextBox.Document;
			LogRichTextBox.Loaded += OnLogRichTextBoxLoaded;
			DataContextChanged += OnDataContextChanged;
        }

		private void OnLogRichTextBoxLoaded(object sender, RoutedEventArgs e)
		{
			if (_scrollViewer != null)
				_scrollViewer.ScrollChanged -= OnScrollChanged;

			_scrollViewer = FindScrollViewer(LogRichTextBox);
			if (_scrollViewer != null)
				_scrollViewer.ScrollChanged += OnScrollChanged;
		}

		private static ScrollViewer FindScrollViewer(DependencyObject obj)
		{
			for (int i = 0; i < VisualTreeHelper.GetChildrenCount(obj); i++)
			{
				var child = VisualTreeHelper.GetChild(obj, i);
				if (child is ScrollViewer sv)
					return sv;
				var result = FindScrollViewer(child);
				if (result != null)
					return result;
			}
			return null;
		}

		private static Brush GetBrush(ConsoleColor color)
			=> (Brush)_colorConverter.Convert(color, typeof(Brush), null, CultureInfo.InvariantCulture);

		private void OnDataContextChanged(object sender, DependencyPropertyChangedEventArgs e)
		{
			if (e.OldValue is JobModel oldModel)
			{
				oldModel.LogLines.CollectionChanged -= OnLogLinesChanged;
				if (_currentLogModel != null)
					_currentLogModel.PropertyChanged -= OnCurrentLogModelChanged;
			}

			_logDocument.Blocks.Clear();
			_currentLogModel = null;
			_currentRun = null;

			if (e.NewValue is JobModel newModel)
			{
				// Populate existing lines (e.g. when DataContext switches to a completed job)
				foreach (var model in newModel.LogLines)
					AppendLogLine(model);

				newModel.LogLines.CollectionChanged += OnLogLinesChanged;
				_autoScroll = true;
				UpdateFollowLink();
			}
		}

		private void AppendLogLine(LogModel model)
		{
			if (_currentLogModel != null)
				_currentLogModel.PropertyChanged -= OnCurrentLogModelChanged;

			var run = new Run(model.Text ?? "");
			run.Foreground = GetBrush(model.Color);
			var para = new Paragraph(run) { Margin = new Thickness(0) };
			_logDocument.Blocks.Add(para);

			_currentRun = run;
			_currentLogModel = model;
			_currentLogModel.PropertyChanged += OnCurrentLogModelChanged;
		}

		private void OnCurrentLogModelChanged(object sender, PropertyChangedEventArgs e)
		{
			var model = (LogModel)sender;
			if (_currentRun == null)
				return;

			if (e.PropertyName == nameof(LogModel.Text))
				_currentRun.Text = model.Text ?? "";
			else if (e.PropertyName == nameof(LogModel.Color))
				_currentRun.Foreground = GetBrush(model.Color);
		}

		private void OnLogLinesChanged(object sender, NotifyCollectionChangedEventArgs e)
		{
			if (e.Action == NotifyCollectionChangedAction.Add)
			{
				foreach (LogModel model in e.NewItems)
					AppendLogLine(model);
			}

			if (_autoScroll)
			{
				_isProgrammaticScroll = true;
				LogRichTextBox.ScrollToEnd();
				_isProgrammaticScroll = false;
			}
		}

		private void OnScrollChanged(object sender, ScrollChangedEventArgs e)
		{
			if (_isProgrammaticScroll)
				return;

			// ExtentHeightChange != 0 means content grew, not a user scroll
			if (e.ExtentHeightChange != 0)
				return;

			_autoScroll = IsAtBottom();
			UpdateFollowLink();
		}

		private bool IsAtBottom()
		{
			if (_scrollViewer == null)
				return true;

			return _scrollViewer.VerticalOffset >= _scrollViewer.ScrollableHeight - 2.0;
		}

		private void UpdateFollowLink()
		{
			FollowLink.Visibility = _autoScroll
				? Visibility.Collapsed
				: Visibility.Visible;
		}

		private void FollowLink_Click(object sender, RoutedEventArgs e)
		{
			_autoScroll = true;
			UpdateFollowLink();
			_isProgrammaticScroll = true;
			LogRichTextBox.ScrollToEnd();
			_isProgrammaticScroll = false;
		}
    }

    public sealed class LogModel : BaseModel
    {
		private string text = null;
        public string Text
		{
			get => text;
			set => SetProperty(ref text, value);
		}

		private ConsoleColor color = ConsoleColor.Black;
        public ConsoleColor Color
		{
			get => color;
			set => SetProperty(ref color, value);
		}
    }

	public sealed class FileModel : BaseModel
	{
		private string name = null;
		public string Name
		{
			get => name;
			set => SetProperty(ref name, value);
		}

		private string status = null;
		public string Status
		{
			get => status;
			set => SetProperty(ref status, value);
		}
	}

    public sealed class JobModel : BaseModel
    {
        public BuildPlatformModel Build { get; }
        public string DstPath { get; }
		public string ScavengePath { get; }
		public string DstPathBase => Path.GetDirectoryName(DstPath);
        public bool DryRun { get; }
        public UnsyncServerConfig Server { get; }
		public string AdditionalArgs { get; }
		public string[] IncludeFilter { get; }
		public string[] Exclusions { get; }

        public string ProxyStr => string.IsNullOrWhiteSpace(Server?.address) ? "(none)" : Server.address;

		public string Name => $"{Build.Name}{(DryRun ? " - Dry Run" : "")}";

		private double totalProgress = 0;
        public double TotalProgress
        {
            get => totalProgress;
            private set => SetProperty(ref totalProgress, value);
        }

        private bool isIndeterminate = true;
        public bool IsIndeterminate
        {
            get => isIndeterminate;
            private set => SetProperty(ref isIndeterminate, value);
        }

        private bool showProgressBar = false;
        public bool ShowProgressBar
        {
            get => showProgressBar;
            private set => SetProperty(ref showProgressBar, value);
        }

        private bool showCancelHyperlink = true;
        public bool ShowCancelHyperlink
        {
            get => showCancelHyperlink;
            private set => SetProperty(ref showCancelHyperlink, value);
        }

        private bool showClearHyperlink = false;
        public bool ShowClearHyperlink
        {
            get => showClearHyperlink;
            private set => SetProperty(ref showClearHyperlink, value);
        }

        private string statusMessage = null;
        public string StatusMessage
        {
            get => statusMessage;
            private set => SetProperty(ref statusMessage, value);
        }

		private ConsoleColor statusColor = ConsoleColor.Black;
		public ConsoleColor StatusColor
		{
			get => statusColor;
			private set => SetProperty(ref statusColor, value);
		}

		private string progressMessage = null;
		public string ProgressMessage
		{
			get => progressMessage;
			private set => SetProperty(ref progressMessage, value);
		}


		private bool detailsExpanded = false;
        public bool DetailsExpanded
        {
            get => detailsExpanded;
            set => SetProperty(ref detailsExpanded, value);
        }

        public Command OnCancelClicked { get; }
        public Command OnClearClicked { get; }
        public Command OnSourcePathClicked { get; }
        public Command OnDestPathClicked { get; }
		public Command CopyLogClicked { get; }
		public Command SaveLogClicked { get; }

		private TaskCompletionSource<bool> startTcs = new TaskCompletionSource<bool>();
        private CancellationTokenSource cts = new CancellationTokenSource();
        public bool IsCancelled => cts.IsCancellationRequested;
		public bool Succeeded { get; private set; }

		private double maxLogScrollViewHeight = Double.PositiveInfinity;
		public double MaxLogScrollViewHeight
		{
			get => maxLogScrollViewHeight;
			set => SetProperty(ref maxLogScrollViewHeight, value);
		}

		private Action<JobModel> onCompletion;
        private Action<JobModel> onProgress;

        public ObservableCollection<LogModel> LogLines { get; } = new ObservableCollection<LogModel>();
		public ObservableCollection<FileModel> Files { get; } = new ObservableCollection<FileModel>();
		private Dictionary<string, FileModel> filesMap = new Dictionary<string, FileModel>();

		public JobModel(
			BuildPlatformModel build,
			string dstPath,
			string scavengePath,
			bool dryRun,
			UnsyncServerConfig server,
			string additionalArgs,
			string[] exclusions,
			Action<JobModel> onCompletion,
			Action<JobModel> onClear,
			Action<JobModel> onProgress)
		{
            Build = build;
            DstPath = dstPath?.TrimEnd('\\');
			ScavengePath = scavengePath?.TrimEnd('\\');
            DryRun = dryRun;
            Server = server;
			AdditionalArgs = additionalArgs;
			IncludeFilter = build.Include?.Split(',', StringSplitOptions.RemoveEmptyEntries)
				.Select(Entry => Entry.Trim())
				.Where(Entry => Entry.Length > 0)
				.ToArray();
			Exclusions = exclusions;
			Succeeded = false;

			OnCancelClicked = new Command(CancelJob) { Enabled = true };
            OnClearClicked = new Command(() => onClear(this)) { Enabled = true };
            OnSourcePathClicked = new Command(() => Shell.LaunchExplorer(Build.FullPath)) { Enabled = true };
            OnDestPathClicked = new Command(() => Shell.LaunchExplorer(DstPath)) { Enabled = true };
			CopyLogClicked = new Command(CopyLog) { Enabled = true };
			SaveLogClicked = new Command(SaveLog) { Enabled = true };

			this.onCompletion = onCompletion;
			this.onProgress = onProgress;

            cts.Token.Register(() => startTcs.TrySetCanceled());

            RunAsync();
        }

		private string GetLogText()
		{
			var builder = new StringBuilder();
			foreach (var logModel in LogLines)
			{
				builder.AppendLine(logModel.Text);
			}
			return builder.ToString();
		}

		private void CopyLog()
		{
			var log = GetLogText();
			if (!string.IsNullOrWhiteSpace(log))
			{
				Clipboard.SetText(log, TextDataFormat.Text);
			}
		}

		private void SaveLog()
		{
			var log = GetLogText();
			if (!string.IsNullOrWhiteSpace(log))
			{
				var defaultName = $"unsync_{Build.SafeName}-{DateTime.Now.ToString("yyyy-MM-dd_HH-mm-ss")}.log";
				var file = Shell.SelectSaveFile("Save Unsync Log", "Log Files|*.log", ".log", defaultName);
				if (file != null)
				{
					try
					{
						File.WriteAllText(file, log);
					}
					catch (Exception ex)
					{
						MessageBox.Show($"Failed to save log to file: {ex}", "Save Log",
							MessageBoxButton.OK, MessageBoxImage.Error);
					}
				}
			}
		}

        public void StartJob() => startTcs.TrySetResult(true);

		public void CancelJob()
		{
			StatusMessage = "Cancelling...";
			OnCancelClicked.Enabled = false;
			IsIndeterminate = true;
			ProgressMessage = null;
			StatusColor = ConsoleColor.Red;
			cts.Cancel();
		}

		public bool IsDuplicate(JobModel otherJob)
		{
			return (Build.Build.Path == otherJob.Build.Build.Path)
				&& (Build.Platform == otherJob.Build.Platform)
				&& (Build.Flavor == otherJob.Build.Flavor)
				&& (DstPathBase == otherJob.DstPathBase);
		}

        private async void RunAsync()
        {
			AsyncProcess proc = null;

			try
			{
				// Wait for the job to be started.
				await startTcs.Task;

				DetailsExpanded = true;
				IsIndeterminate = true;
				ShowProgressBar = true;

				StatusMessage = "Starting...";

				LogModel currentLine = null;
				void EmitLog(string line)
				{
					if (currentLine == null)
					{
						LogLines.Add(currentLine = new LogModel());
					}

					currentLine.Text = line;

					// Determine log color
					var color = ConsoleColor.Black;
					if (currentLine.Text.Contains("ERROR:")) { color = ConsoleColor.Red; }
					else if (currentLine.Text.Contains("WARNING:")) { color = ConsoleColor.DarkYellow; }

					currentLine.Color = color;
				}

				void ProcessCommand(string message)
				{
					// Don't process command messages if we're cancelling, otherwise they'll overwrite UI state
					if (cts.Token.IsCancellationRequested)
						return;

					var progressPattern = new Regex(@"@progress \[([^\]]*)\] ([0-9]+) / ([0-9]+)");
					var statusPattern = new Regex(@"@status \[([^\]]*)\] (.*)");

					Match match;
					if ((match = progressPattern.Match(message)).Success)
					{
						var target = match.Groups[1].Value;
						var numerator = ulong.Parse(match.Groups[2].Value);
						var denominator = ulong.Parse(match.Groups[3].Value);

						if (target == "*")
						{
							// Overall progress
							TotalProgress = denominator > 0.0 ? Math.Clamp((numerator / (double)denominator) * 100.0, 0.0, 100.0) : 0.0;
							ProgressMessage = $"{TotalProgress:0.00} %";
							IsIndeterminate = false;

							onProgress(this);
						}
						else
						{
							// @todo: specific file progress
						}
					}
					else if ((match = statusPattern.Match(message)).Success)
					{
						var target = match.Groups[1].Value;
						var text = match.Groups[2].Value;

						if (target == "*")
						{
							// Overall status
							StatusMessage = text;
						}
						else
						{
							if (!filesMap.TryGetValue(target, out var file))
							{
								filesMap.Add(target, file = new FileModel() { Name = target });
								Files.Add(file);
							}

							file.Status = text;
						}
					}
				}

				var commandBuilder = new StringBuilder();
				var lineBuilder = new StringBuilder();

				char previous = (char)0;
				bool command = false;

				// Launch unsync.exe
				var args = new List<string>() { "-v", "--progress" };
				if (App.Current.UnsyncSupportsStdout)
				{
					args.Add("--stdout");
				}
				if (DryRun)
				{
					args.Add("-d");
				}

				if (Server != null)
				{
					args.Add(Server.GetCommandLineArgs());

					if (App.Current.Config.EnableUserAuthentication
						&& App.Current.Config.loggedInUser != null)
					{
						args.Add("--login");
					}
				}

				if (IncludeFilter?.Length > 0)
				{
					args.Add($"--include {string.Join(",", IncludeFilter)}");
				}
				if (Exclusions != null && Exclusions.Length > 0)
				{
					args.Add($"--exclude {string.Join(",", Exclusions)}");
				}

				if (ScavengePath != null)
				{
					args.Add($"--scavenge \"{ScavengePath}\"");
				}

				args.Add($"\"{Build.FullPath}\"");
				args.Add($"\"{DstPath}\"");

				var argsStr = $"sync{(AdditionalArgs != null ? $" {AdditionalArgs}" : "")} {string.Join(" ", args)}";

				// Allow completely overriding command line arguments (useful for testing)
				if (AdditionalArgs != null && AdditionalArgs.StartsWith("!"))
				{
					argsStr = AdditionalArgs.Substring(1);
				}

				App.Current.LogMessage($"Running unsync with args {argsStr}");

				var unsyncPath = App.Current.Config.UnsyncPath;
				proc = new AsyncProcess(unsyncPath, argsStr);

				await foreach (var str in proc.RunAsync(cts.Token))
				{
					foreach (var current in str)
					{
						if (current == '@' && (previous == '\r' || previous == '\n' || previous == (char)0))
						{
							command = true;
						}

						var builder = command
							? commandBuilder
							: lineBuilder;

						switch (current)
						{
							case '\n':
								if (command)
								{
									ProcessCommand(builder.ToString());
									command = false;
								}
								else
								{
									if (previous != '\r')
									{
										EmitLog(builder.ToString());
									}

									currentLine = null;
								}
								builder.Clear();
								break;

							case '\r':
								if (!command)
								{
									EmitLog(builder.ToString());
									builder.Clear();
								}
								break;

							default:
								builder.Append(current);
								break;
						}

						previous = current;
					}
				}
			}
			catch (OperationCanceledException)
			{
			}
			finally
			{
				ShowProgressBar = false;
				ShowCancelHyperlink = false;
				ShowClearHyperlink = true;

				if (proc?.ExitCode == 0)
				{
					StatusMessage = "Completed";
					StatusColor = ConsoleColor.DarkGreen;
					Succeeded = true;
				}
				else if (proc?.ExitCode != null)
				{
					StatusMessage = $"Failed (error code {proc?.ExitCode})";
					StatusColor = ConsoleColor.Red;
				}
				else
				{
					StatusMessage = $"Failed (unknown error)";
					StatusColor = ConsoleColor.Red;
				}

				ProgressMessage = null;
				DetailsExpanded = false;

				onCompletion(this);

				// Allow the log to fill the entire window while job is running, but set a concrete limit on completed jobs.
				// This is a workaround for missing scroll bar in the completed job logs.
				MaxLogScrollViewHeight = 300;
			}
        }
    }
}
