<!--Copyright Epic Games, Inc. All Rights Reserved.-->

<Window x:Class="UnsyncUI.MainWindow"
        xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
        xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
        xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
        xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
        xmlns:local="clr-namespace:UnsyncUI"
        mc:Ignorable="d"
        Title="UnsyncUI"
        Height="800"
        Width="1600"
        MinWidth="600"
        MinHeight="400" 
		Closing="Window_Closing">
	<Window.TaskbarItemInfo>
		<TaskbarItemInfo ProgressState="{Binding ProgressState}" ProgressValue="{Binding ProgressValue}"/>
	</Window.TaskbarItemInfo>
	<Window.Resources>

		<local:ConsoleColorConverter x:Key="ConsoleColorConverter"/>

		<local:MainTabTemplateSelector x:Key="mainTabSelector">
			<local:MainTabTemplateSelector.ProjectTemplate>
				<DataTemplate>
					<Grid>
						<Grid.RowDefinitions>
							<RowDefinition Height="auto" />
							<RowDefinition Height="*" />
						</Grid.RowDefinitions>

						<local:BuildSelectorControl Grid.Row="1" />

					</Grid>
				</DataTemplate>
			</local:MainTabTemplateSelector.ProjectTemplate>

			<!-- Contents of the "Custom" project tab -->
			<local:MainTabTemplateSelector.CustomTemplate>
				<DataTemplate>
					<Grid>
						<Grid.RowDefinitions>
							<RowDefinition Height="auto"/>
							<RowDefinition Height="auto"/>
							<RowDefinition Height="auto"/>
							<RowDefinition Height="auto"/>
							<RowDefinition Height="*"/>
						</Grid.RowDefinitions>

						<GroupBox Grid.Row="0" Header="Source">
							<local:SelectFolderControl SelectedPath="{Binding SrcPath}" Description="Select a source directory to sync builds from."/>
						</GroupBox>

						<GroupBox Grid.Row="1" Header="Destination">
							<local:SelectFolderControl SelectedPath="{Binding DstPath}" Description="Select a destination directory to sync builds to."/>
						</GroupBox>

						<GroupBox Grid.Row="2" Header="Include">
							<TextBox Text="{Binding Include}" ToolTip="Comma-separated list parts of filenames to filter the download"/>
						</GroupBox>

						<Grid Grid.Row="3" >
							<CheckBox Content="Dry Run" IsChecked="{Binding DryRun}" ToolTip="Download and verify the data, but don't write outputs to disk" Margin="10,4,4,4" HorizontalAlignment="Left"/>
							<TextBlock Margin="4" HorizontalAlignment="Center">
								<Hyperlink Command="{Binding OnSyncClicked}">Start Custom Sync</Hyperlink>
							</TextBlock>
						</Grid>

					</Grid>
				</DataTemplate>
			</local:MainTabTemplateSelector.CustomTemplate>

		</local:MainTabTemplateSelector>

	</Window.Resources>
	<Grid>
		<Grid.RowDefinitions>
			<RowDefinition Height="auto"/>
			<RowDefinition Height="*"/>
		</Grid.RowDefinitions>

		<!-- Announcement Banner — hidden when no announcements -->
		<Border Grid.Row="0"
				Visibility="{Binding AnnouncementBanner.IsVisible, Converter={StaticResource VisConv_Collapsed}}"
				BorderBrush="LightGray" BorderThickness="0,0,0,1" Padding="8,4" Margin="4,4,4,0">
			<Grid>
				<Grid.ColumnDefinitions>
					<ColumnDefinition Width="*"/>
					<ColumnDefinition Width="auto"/>
				</Grid.ColumnDefinitions>
				<ItemsControl Grid.Column="0" ItemsSource="{Binding AnnouncementBanner.Announcements}">
					<ItemsControl.ItemTemplate>
						<DataTemplate>
							<TextBlock Text="{Binding Message}" TextWrapping="Wrap"
									   Foreground="{Binding Color, Converter={StaticResource ConsoleColorConverter}}"
									   Margin="0,2"/>
						</DataTemplate>
					</ItemsControl.ItemTemplate>
				</ItemsControl>
				<TextBlock Grid.Column="1" VerticalAlignment="Top" Margin="8,2,0,0">
					<Hyperlink Command="{Binding AnnouncementBanner.OnDismissClicked}">Dismiss</Hyperlink>
				</TextBlock>
			</Grid>
		</Border>

		<Grid Grid.Row="1" Margin="4">
		<Grid.ColumnDefinitions>
			<ColumnDefinition Width="588"/>
			<ColumnDefinition Width="8"/>
			<ColumnDefinition Width="*"/>
		</Grid.ColumnDefinitions>

		<!--Left panel-->
		<Grid Grid.Row="0">
			<Grid.RowDefinitions>
				<RowDefinition Height="auto" />
				<RowDefinition Height="auto" />
				<RowDefinition Height="*" />
			</Grid.RowDefinitions>
			<GroupBox Grid.Row="0" Header="Server">
				<Grid>
					<Grid.RowDefinitions>
						<RowDefinition Height="auto" />
						<RowDefinition Height="auto" />
					</Grid.RowDefinitions>
					<ComboBox Grid.Row="0" ItemsSource="{Binding Config.Proxies}" SelectedItem="{Binding SelectedProxy}">
						<ComboBox.ItemTemplate>
							<DataTemplate>
								<TextBlock>
									<Run Text="{Binding Name}" />
									<TextBlock Visibility="{Binding Path, Converter={StaticResource VisConv_Collapsed}}">
										<Run Text=" ("/>
										<Run Text="{Binding Path}"/>
										<Run Text=")"/>
									</TextBlock>
							</TextBlock>
							</DataTemplate>
						</ComboBox.ItemTemplate>
					</ComboBox>
					<Grid Grid.Row="1" ToolTip="User authentication allows enumerating files and downloading data directly through the proxy server">
						<Grid.ColumnDefinitions>
							<ColumnDefinition Width="auto"/>
							<ColumnDefinition Width="*"/>
						</Grid.ColumnDefinitions>
						<TextBlock Grid.Column="0" Visibility="{Binding Path=ShouldShowLoginInfo, Converter={StaticResource VisConv_Collapsed}}">
							<Run Text="User:"/>
							<!--Logged in-->
							<TextBlock Visibility="{Binding Path=LoggedInUser, Converter={StaticResource VisConv_Collapsed}}">
								<Run FontWeight="Bold" Text="{Binding Path=LoggedInUser, Mode=OneWay}"/>
								<Run Text=" - "/>
								<Hyperlink Command="{Binding OnLogOutClicked}">Log out</Hyperlink>
							</TextBlock>
							<!--Not logged in-->
							<TextBlock Visibility="{Binding Path=LoggedInUser, Converter={StaticResource VisConv_Collapsed_Inv}}">
								<Hyperlink FontWeight="Bold" Command="{Binding OnLogInClicked}">Log in</Hyperlink>
							</TextBlock>
						</TextBlock>
					</Grid>
				</Grid>
			</GroupBox>

			<GroupBox Grid.Row="1" Header="Additional arguments">
				<Grid>
					<TextBox Grid.Row="1" Grid.Column="0" Grid.ColumnSpan="2" Text="{Binding AdditionalArgs}" />
				</Grid>
			</GroupBox>

			<GroupBox Grid.Row="2" Header="Projects">
				<Grid>
					<Grid.RowDefinitions>
						<RowDefinition Height="auto"/>
						<RowDefinition Height="*"/>
					</Grid.RowDefinitions>
					<TabControl Grid.Row ="1" ItemsSource="{Binding Tabs}" SelectedIndex="0" ContentTemplateSelector="{Binding Source={StaticResource mainTabSelector}}" SelectedItem="{Binding SelectedTab}">
						<TabControl.ItemTemplate>
							<DataTemplate>
								<TextBlock Text="{Binding Name}"/>
							</DataTemplate>
						</TabControl.ItemTemplate>
					</TabControl>
				</Grid>
			</GroupBox>

		</Grid>

		<GridSplitter Grid.Column="1" Grid.RowSpan="3" HorizontalAlignment="Stretch" Foreground="Transparent" Background="Transparent" />

		<!--Right panel-->
		<Grid Grid.Row="0" Grid.Column="2" >
			<Grid.RowDefinitions>
				<RowDefinition Height="*"/>
				<RowDefinition Height="4"/>
				<RowDefinition Height="100"/>
			</Grid.RowDefinitions>
			<!-- Sync job logs -->
			<Grid Grid.Row="0">
				<!-- Help text -->
				<TextBlock Grid.Row="0" Grid.Column="2" Grid.RowSpan="3" TextAlignment="Center" VerticalAlignment="Center">
					<TextBlock.Visibility>
						<MultiBinding Converter="{StaticResource VisConv_All_Collapsed}">
							<Binding Path="QueuedJobs.Count" Converter="{StaticResource NotConv}" />
							<Binding Path="ActiveJob" Converter="{StaticResource NotConv}" />
							<Binding Path="CompletedJobs.Count" Converter="{StaticResource NotConv}" />
						</MultiBinding>
					</TextBlock.Visibility>
					<Run Text="To get started, search for builds in the left pane,"/>
					<LineBreak/>
					<Run Text="select one or more platforms and a destination folder, and click Start Sync."/>
				</TextBlock>
				<!-- Job logs -->
				<Grid Grid.Row="0" Grid.Column="2" Grid.RowSpan="3">
					<Grid.Resources>
						<GridLength x:Key="glStar">*</GridLength>
						<GridLength x:Key="glAuto">auto</GridLength>
						<local:ObjectConverter x:Key="rdHeightConv" TrueValue="{StaticResource glStar}" FalseValue="{StaticResource glAuto}"/>
						<local:ObjectConverter x:Key="rdMinHeightConv" TrueValue="60" FalseValue="0"/>
					</Grid.Resources>
					<Grid.RowDefinitions>
						<RowDefinition Height="auto"/>
						<RowDefinition Height="{Binding ActiveJob, Converter={StaticResource rdHeightConv}}" MinHeight="{Binding ActiveJob, Converter={StaticResource rdMinHeightConv}}"/>
						<RowDefinition Height="auto"/>
					</Grid.RowDefinitions>

					<!-- Queued Jobs Pane -->
					<GroupBox Grid.Row="0" Visibility="{Binding QueuedJobs.Count, Converter={StaticResource VisConv_Collapsed}}">
						<GroupBox.Header>
							<TextBlock VerticalAlignment="Center" HorizontalAlignment="Center">
									<Run Text="{Binding QueuedJobs.Count, Mode=OneWay}" />
									<Run Text="Queued Job(s)" />
									<TextBlock>
										<Run Text=" - " />
										<Hyperlink Command="{Binding OnClearQueueClicked}">Cancel All</Hyperlink>
									</TextBlock>
								</TextBlock>
						</GroupBox.Header>
						<ItemsControl ItemsSource="{Binding QueuedJobs}">
							<ItemsControl.ItemTemplate>
								<ItemContainerTemplate>
									<local:JobControl/>
								</ItemContainerTemplate>
							</ItemsControl.ItemTemplate>
						</ItemsControl>
					</GroupBox>

					<!-- Active Job Pane -->
					<GroupBox Grid.Row="1" Visibility="{Binding ActiveJob, Converter={StaticResource VisConv_Collapsed}}" Header="Active Job">
						<local:JobControl DataContext="{Binding ActiveJob}"/>
					</GroupBox>

					<!-- Completed Jobs Pane -->
					<GroupBox Grid.Row="2"  Visibility="{Binding CompletedJobs.Count, Converter={StaticResource VisConv_Collapsed}}">
						<GroupBox.Header>
							<TextBlock VerticalAlignment="Center" HorizontalAlignment="Center">
								<Run Text="{Binding CompletedJobs.Count, Mode=OneWay}" />
								<Run Text="Completed Job(s)" />
								<TextBlock Visibility="{Binding CompletedJobs.Count, Converter={StaticResource VisConv_Collapsed}}">
									<Run Text=" - " />
									<Hyperlink Command="{Binding OnClearCompletedClicked}">Clear All</Hyperlink>
								</TextBlock>
							</TextBlock>
						</GroupBox.Header>

						<ItemsControl ItemsSource="{Binding CompletedJobs}">
							<ItemsControl.ItemTemplate>
								<ItemContainerTemplate>
									<local:JobControl/>
								</ItemContainerTemplate>
							</ItemsControl.ItemTemplate>
						</ItemsControl>
					</GroupBox>
				</Grid>
			</Grid>
			<GridSplitter Grid.Row="1" HorizontalAlignment="Stretch" />
			<!-- Main app log -->
			<Grid Grid.Row="2">
				<GroupBox>
					<GroupBox.Header>
						<WrapPanel>
							<!--
								TODO: expandable log window
								<Expander VerticalAlignment="Center" Name="ApplicationLogExpander" IsExpanded="{Binding LogExpanded}" />
							-->
							<TextBlock VerticalAlignment="Center"><Run Text="Application Log"/>
								<Run Text=" - " />
								<Hyperlink Command="{Binding OnClearApplicationLogClicked}">Clear</Hyperlink>
							</TextBlock>
						</WrapPanel>
					</GroupBox.Header>
					<local:AutoScrollTextBox 
						Text="{Binding ApplicationLog, Mode=OneWay}" 
						IsReadOnly="True"
						IsReadOnlyCaretVisible="True"
						VerticalScrollBarVisibility="Visible"
						HorizontalScrollBarVisibility="Auto"
						TextWrapping="Wrap"
						>
					</local:AutoScrollTextBox>
				</GroupBox>
			</Grid>
		</Grid>

	</Grid>
	</Grid>

</Window>
