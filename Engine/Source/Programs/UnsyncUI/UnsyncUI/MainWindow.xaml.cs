// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.ObjectModel;
using System.ComponentModel;
using System.Diagnostics;
using System.Linq;
using System.Windows;
using System.Windows.Controls;
using System.Windows.Shell;

namespace UnsyncUI
{
	/// <summary>
	/// Interaction logic for MainWindow.xaml
	/// </summary>
	public partial class MainWindow : Window
	{
		public MainWindow()
		{
			InitializeComponent();
			DataContext = new MainWindowModel();
			var version = System.Reflection.Assembly.GetEntryAssembly().GetName().Version;
			Title = $"UnsyncUI v{version.Major}.{version.Minor}.{version.Build}";

			if (App.Current.EnableExperimentalFeatures)
			{
				Title += " [EXPERIMENTAL]";
			}
		}

		private void Window_Closing(object sender, CancelEventArgs e)
		{
			var model = DataContext as MainWindowModel;
			if (model != null)
			{
				if (model.ActiveJob != null || model.QueuedJobs.Count > 0)
				{
					if (MessageBox.Show(
						"There are active transfers in progress. Are you sure you want to quit? Transfers will be aborted.", 
						"Quit Unsync",
						MessageBoxButton.YesNo,
						MessageBoxImage.Warning) != MessageBoxResult.Yes)
					{
						e.Cancel = true;
					}
				}
			}
		}
	}

	public class AutoScrollTextBox : TextBox
	{
		protected override void OnTextChanged(TextChangedEventArgs e)
		{
			base.OnTextChanged(e);
			ScrollToEnd();
		}
	}

	public sealed class MainTabTemplateSelector : DataTemplateSelector
	{
		public DataTemplate ProjectTemplate { get; set; }
		public DataTemplate CustomTemplate { get; set; }

		public override DataTemplate SelectTemplate(object item, DependencyObject container)
			=> item is ProjectModel
				? ProjectTemplate
				: CustomTemplate;
	}

	public sealed class AnnouncementItem : BaseModel
	{
		public string Message { get; }
		public ConsoleColor Color { get; }

		public AnnouncementItem(string message, string level)
		{
			Message = (message ?? "").Replace("\r\n", " ").Replace("\n", " ").Replace("\r", " ");
			Color = level switch
			{
				"error" => ConsoleColor.Red,
				"warning" => ConsoleColor.DarkYellow,
				_ => ConsoleColor.Black,
			};
		}
	}

	public sealed class AnnouncementBannerModel : BaseModel
	{
		private HashSet<string> seenIds = new HashSet<string>();

		public ObservableCollection<AnnouncementItem> Announcements { get; } = new ObservableCollection<AnnouncementItem>();

		private bool isVisible = false;
		public bool IsVisible
		{
			get => isVisible;
			set => SetProperty(ref isVisible, value);
		}

		public Command OnDismissClicked { get; }

		public AnnouncementBannerModel()
		{
			OnDismissClicked = new Command(() => IsVisible = false) { Enabled = true };
		}

		public void AddAnnouncements(IEnumerable<HelloAnnouncement> announcements)
		{
			foreach (var a in announcements)
			{
				if (a.id != null && !seenIds.Add(a.id))
				{
					continue;
				}

				Announcements.Add(new AnnouncementItem(a.message, a.level));
			}

			if (Announcements.Count > 0)
			{
				IsVisible = true;
			}
		}
	}

	public abstract class TabModel : BaseModel
	{
		public virtual void OnSelected() { }
	}

	public sealed class CustomModel : TabModel
	{
		private Action<IEnumerable<(SyncStartConfig Config, BuildPlatformModel Model)>> onBuildsSelected;

		public string Name => "Custom";

		private string srcPath = App.Current.UserConfig.CustomSrcPath;
		public string SrcPath
		{
			get => srcPath;
			set
			{
				SetProperty(ref srcPath, value);
				App.Current.UserConfig.CustomSrcPath = value;

				UpdateSyncCommand();
			}
		}

		public bool DryRun { get; set; }

		private string dstPath = App.Current.UserConfig.CustomDstPath;
		public string DstPath
		{
			get => dstPath;
			set
			{
				SetProperty(ref dstPath, value);
				App.Current.UserConfig.CustomDstPath = value;

				UpdateSyncCommand();
			}
		}

		private string include = App.Current.UserConfig.CustomInclude;
		public string Include
		{
			get => include;
			set
			{
				SetProperty(ref include, value);
				App.Current.UserConfig.CustomInclude = value;
			}
		}

		public Command OnSyncClicked { get; }

		private void UpdateSyncCommand()
			=> OnSyncClicked.Enabled = !string.IsNullOrWhiteSpace(SrcPath) && !string.IsNullOrWhiteSpace(DstPath);

		public CustomModel(Action<IEnumerable<(SyncStartConfig Config, BuildPlatformModel Model)>> onBuildsSelected)
		{
			this.onBuildsSelected = onBuildsSelected;
			OnSyncClicked = new Command(() =>
			{
				var config = new SyncStartConfig();
				config.DryRun = DryRun;
				config.DstPath = DstPath;
				config.Exclusions = default(string[]);
				onBuildsSelected(new[]
				{
					(config, new BuildPlatformModel(null, null, SrcPath, null, Include))
				});
			});

			UpdateSyncCommand();
		}
	}

	public class SyncStartConfig
	{
		public bool DryRun;
		public string DstPath;
		public string ScavengePath;
		public string[] Exclusions;
	};

	public sealed class MainWindowModel : BaseModel
	{
		public AnnouncementBannerModel AnnouncementBanner { get; } = new AnnouncementBannerModel();

		private Config.Proxy selectedProxy =
			App.Current.Config?.Proxies?.Where(p => p.Name.Equals(App.Current.UserConfig.Proxy, StringComparison.InvariantCultureIgnoreCase))?.FirstOrDefault() ??
			App.Current.Config?.Proxies?.Where(p => p.Path != null).FirstOrDefault();

		public Config.Proxy SelectedProxy
		{
			get => selectedProxy;
			set
			{
				SetProperty(ref selectedProxy, value);
				App.Current.UserConfig.Proxy = value.Name;
			}
		}

		private bool logExpanded = true;
		public bool LogExpanded
		{
			get => logExpanded;
			set => SetProperty(ref logExpanded, value);
		}

		public string ApplicationLog
		{
			get => App.Current.ApplicationLog;
		}

		public void OnLogUpdated()
		{
			OnPropertyChanged("ApplicationLog");
		}

		private bool showHelp = true;
		public bool ShowHelp
		{
			get => showHelp;
			set => SetProperty(ref showHelp, value);
		}

		private string additionalArgs = App.Current.UserConfig.AdditionalArgs;
		public string AdditionalArgs
		{
			get => additionalArgs;
			set
			{
				SetProperty(ref additionalArgs, value);
				App.Current.UserConfig.AdditionalArgs = value;
			}
		}

		public bool ShouldShowLoginInfo { get => App.Current.EnableUserAuthentication; }

		public string LoggedInUser
		{
			get => Config?.loggedInUser;
			set 
			{
				App.Current.UserConfig.LogInOnStartup = (value != null);
				SetProperty(ref Config.loggedInUser, value);
			}
		}

		public Command OnLogInClicked { get; }
		public Command OnLogOutClicked { get; }

		private void LogIn()
		{
			if (Config.RootProxy?.Path == null
				|| Config.UnsyncPath == null)
			{
				LoggedInUser = null;
				return;
			}

			UnsyncQueryUtil queryUtil = new UnsyncQueryUtil(Config.UnsyncPath, Config.RootProxy.Path, Config.RootProxy.Protocol);

			try
			{
				LoginQueryResult LoginInfo = queryUtil.Login();
				LoggedInUser = LoginInfo.sub;
			}
			catch (Exception ex)
			{
				App.Current.LogError($"Login failed with exception: {ex}");
			}
		}

		private void LogOut()
		{
			LoggedInUser = null;
		}


		private TaskbarItemProgressState progressState = TaskbarItemProgressState.None;
		public TaskbarItemProgressState ProgressState
		{
			get => progressState;
			set => SetProperty(ref progressState, value);
		}

		private float progressValue = 0.0f;
		public float ProgressValue
		{
			get => progressValue;
			set => SetProperty(ref progressValue, value);
		}

		private int batchJobCount = 0;

		public Config Config => (Application.Current as App).Config;

		public Command OnClearQueueClicked { get; }
		public Command OnClearCompletedClicked { get; }

		public Command OnClearApplicationLogClicked { get; }

		public ObservableCollection<TabModel> Tabs { get; } = new ObservableCollection<TabModel>();
		public ObservableCollection<JobModel> QueuedJobs { get; } = new ObservableCollection<JobModel>();

		private JobModel activeJob = null;
		public JobModel ActiveJob 
		{
			get => activeJob;
			set => SetProperty(ref activeJob, value);
		}

		public ObservableCollection<JobModel> CompletedJobs { get; } = new ObservableCollection<JobModel>();

		private TabModel selectedTab = null;
		public TabModel SelectedTab
		{
			get => selectedTab;
			set
			{
				SetProperty(ref selectedTab, value);
				selectedTab?.OnSelected();
			}
		}

		public MainWindowModel()
		{
			OnClearQueueClicked = new Command(ClearQueue) { Enabled = true };
			OnClearCompletedClicked = new Command(ClearCompleted) { Enabled = true };
			OnClearApplicationLogClicked = new Command(ClearApplicationLog) { Enabled = true };

			OnLogInClicked = new Command(LogIn) { Enabled = true };
			OnLogOutClicked = new Command(LogOut) { Enabled = true };

			if (Config != null)
			{
				foreach (var p in Config.Projects)
				{
					Tabs.Add(new ProjectModel(p, OnBuildsSelected));
				}

				if (Config.EnableUserAuthentication && App.Current.UserConfig.LogInOnStartup)
				{
					LogIn();
				}
			}

			Tabs.Add(new CustomModel(OnBuildsSelected));

			SelectedTab = Tabs.FirstOrDefault();

			FetchAnnouncements();
		}

		private void FetchAnnouncements()
		{
			if (Config?.RootProxy?.Path == null || Config?.UnsyncPath == null)
			{
				return;
			}

			System.Threading.Tasks.Task.Run(() =>
			{
				try
				{
					var queryUtil = new UnsyncQueryUtil(Config.UnsyncPath, Config.RootProxy.GetServerConfig());
					var announcements = queryUtil.Hello();

					if (announcements.Count > 0)
					{
						App.Current.Dispatcher.InvokeAsync(() =>
						{
							AnnouncementBanner.AddAnnouncements(announcements);
						});
					}
				}
				catch (Exception ex)
				{
					App.Current.LogDebug($"Failed to fetch announcements: {ex.Message}");
				}
			});
		}

		private void ClearQueue()
		{
			foreach (var job in QueuedJobs)
				job.CancelJob();
		}

		private void ClearCompleted()
		{
			CompletedJobs.Clear();
			UpdateProgressState();
		}

		private void ClearApplicationLog()
		{
			App.Current.ClearApplicationLog();
		}

		public void OnBuildsSelected(IEnumerable<(SyncStartConfig Config, BuildPlatformModel Model)> selectedBuilds)
		{
			foreach (var build in selectedBuilds)
			{
				AddJob(new JobModel(
					build.Model, 
					build.Config.DstPath,
					build.Config.ScavengePath,
					build.Config.DryRun, 
					SelectedProxy?.GetServerConfig(),
					AdditionalArgs, 
					build.Config.Exclusions, 
					OnJobCompleted, 
					OnClearJob, 
					OnJobProgress
				));
			}
		}

		private void AddJob(JobModel job)
		{
			++batchJobCount;

			if (ActiveJob == null)
			{
				ActiveJob = job;
				job.StartJob();
			}
			// Don't bother including a job that is already either active or queued up :
			else if (!ActiveJob.IsDuplicate(job) && (QueuedJobs.FirstOrDefault(j => j.IsDuplicate(job)) == null))
			{
				// Add it after the last platform for that same build if any, so that jobs from the same build end up being processed one after another 
				JobModel lastJobForBuild = QueuedJobs.LastOrDefault(j => (j.Build.Build.Path == job.Build.Build.Path) && (j.DstPathBase == job.DstPathBase));
				int insertionIndex = 0;
				if (lastJobForBuild != null)
				{
					insertionIndex = QueuedJobs.IndexOf(lastJobForBuild) + 1;
				}
				QueuedJobs.Insert(insertionIndex, job);
			}

			UpdateProgressValue();
			UpdateProgressState();
		}

		private void OnJobProgress(JobModel job)
		{
			UpdateProgressValue();
		}

		private void OnJobCompleted(JobModel job)
		{
			QueuedJobs.Remove(job);

			if (!job.IsCancelled)
			{
				CompletedJobs.Add(job);
			}
			else
			{
				Debug.Assert(batchJobCount > 0);
				--batchJobCount;
			}

			if (job == ActiveJob)
			{
				ActiveJob = null;
				if (QueuedJobs.Count > 0)
				{
					// Look for a new job to run. Pick the first one rather than the last one for the queue to act as FIFO : 
					var newJob = QueuedJobs.Where(q => !q.IsCancelled).FirstOrDefault();
					if (newJob != null)
					{
						QueuedJobs.Remove(newJob);
						ActiveJob = newJob;

						newJob.StartJob();
					}
				}
				else
				{
					batchJobCount = 0;
				}
			}

			UpdateProgressValue();
			UpdateProgressState();
		}

		private void OnClearJob(JobModel job)
		{
			CompletedJobs.Remove(job);
			UpdateProgressState();
		}

		private void UpdateProgressState()
		{
			bool error = false;

			foreach (var job in CompletedJobs)
			{
				if (!job.Succeeded)
					error = true;
			}

			if(error)
			{
				ProgressState = TaskbarItemProgressState.Error;
			}
			else if(batchJobCount > 0)
			{
				ProgressState = TaskbarItemProgressState.Normal;
			}
			else
			{
				ProgressState = TaskbarItemProgressState.None;
			}
		}
		private void UpdateProgressValue()
		{
			if(batchJobCount == 0)
			{
				ProgressValue = 1.0f;
			}
			else
			{
				float progress = batchJobCount - QueuedJobs.Count;

				if(ActiveJob != null)
				{
					progress += (float)ActiveJob.TotalProgress / 100.0f - 1;
				}

				ProgressValue = progress / batchJobCount;
			}
		}
	}
}
