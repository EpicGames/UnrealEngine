// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using System.Threading;
using System;
using System.IO;
using System.Diagnostics;
using System.Text.Json;
using System.Linq;
using System.Collections;
using System.Text.Json.Serialization;

namespace UnsyncUI
{
	public class UnsyncServerConfig
	{
		public String address;
		public String protocol;

		public String GetCommandLineArgs()
		{
			List<String> args = new List<String>();

			if (!string.IsNullOrWhiteSpace(address))
			{
				args.Add($"--server {address}");
			}

			if (!string.IsNullOrWhiteSpace(protocol))
			{
				args.Add($"--protocol {protocol}");
			}

			return string.Join(" ", args);
		}
	}

	public class UnsyncQueryConfig
	{
		public String unsyncPath;
		public UnsyncServerConfig server;
	}

	class SearchQueryResultEntry
	{
		public String path { get; set; }

		public bool is_directory { get; set; }
		public UInt64 size { get; set; }
		public UInt64 mtime { get; set; }
	}

	class SearchQueryResult
	{
		public String root { get; set; }
		public List<SearchQueryResultEntry> entries { get; set; }
	}

	public class HelloAnnouncement
	{
		public string id { get; set; }
		public string level { get; set; }
		public string message { get; set; }
	}

	class HelloResponse
	{
		public List<HelloAnnouncement> announcements { get; set; }
	}

	// Subset of decoded JWT claims
	public class LoginQueryResult
	{
		[JsonConverter(typeof(SingleOrArrayConverter))]
		public List<String> aud { get; set; }
		public String sub { get; set; }
		public String iss { get; set; }
		public List<String> scp { get; set; }
		public UInt64 iat { get; set; }
		public UInt64 exp { get; set; }
		public List<String> groups { get; set; }
	}

	public class UnsyncMirrorDesc
	{
		public String name { get; set; }
		public String description { get; set; }
		public String address { get; set; }
		public uint port { get; set; }
		public uint ping { get; set; }
		public bool ok { get; set; }
	}

	public class UnsyncQueryUtil
	{
		UnsyncQueryConfig Config;
		public UnsyncQueryUtil(UnsyncQueryConfig InConfig)
		{
			Config = InConfig;
		}

		public UnsyncQueryUtil(string unsyncPath, string serverAddress, string serverProtocol)
			: this(unsyncPath, new UnsyncServerConfig { address = serverAddress, protocol = serverProtocol } )
		{
		}

		public UnsyncQueryUtil(string unsyncPath, UnsyncServerConfig server)
		{
			Config = new UnsyncQueryConfig();
			Config.unsyncPath = unsyncPath;
			Config.server = server;
		}

		private string RunCommand(string argsStr)
		{
			AsyncProcess proc = new AsyncProcess(Config.unsyncPath, argsStr);
			CancellationToken cancellationToken = new CancellationToken();

			var response = "";
			var diagnostics = "";

			var QueryTask = Task.Run(async () => {
				await foreach (var (str, kind) in proc.RunAsyncStreams(cancellationToken))
				{
					if (kind == AsyncProcess.StreamKind.StdOut)
					{
						response += str;
					}
					else
					{
						diagnostics += str;
					}
				}
			});

			QueryTask.Wait();

			if (proc.ExitCode == 0)
			{
				return response;
			}
			else
			{
				throw new Exception($"Exit code {proc.ExitCode}\n{diagnostics}");
			}
		}

		public List<UnsyncMirrorDesc> Mirrors()
		{
			string responseJson = RunCommand($"query mirrors {Config.server.GetCommandLineArgs()}");
			return JsonSerializer.Deserialize<List<UnsyncMirrorDesc>>(responseJson);
		}

		public LoginQueryResult Login()
		{
			string responseJson = RunCommand($"login --decode {Config.server.GetCommandLineArgs()}");
			return JsonSerializer.Deserialize<LoginQueryResult>(responseJson);
		}

		public List<HelloAnnouncement> Hello()
		{
			string serverAddress = Config.server?.address;
			if (string.IsNullOrWhiteSpace(serverAddress))
			{
				return new List<HelloAnnouncement>();
			}

			string scheme = serverAddress.EndsWith(":443") || serverAddress.Contains(":443/") ? "https" : "http";
			string uriString = serverAddress.Contains("://") ? serverAddress : $"{scheme}://{serverAddress}";
			if (!Uri.TryCreate(uriString, UriKind.Absolute, out Uri baseUri))
			{
				return new List<HelloAnnouncement>();
			}
			string url = new Uri(baseUri, "/api/v1/hello").ToString();
			string responseJson = RunCommand($"query http-get {url}");
			HelloResponse response = JsonSerializer.Deserialize<HelloResponse>(responseJson);
			return response?.announcements ?? new List<HelloAnnouncement>();
		}
	}

	

	public class UnsyncDirectoryEnumerator : IDirectoryEnumerator
	{
		private UnsyncQueryConfig Config;

		private Config.Project ProjectSchema;

		private String DirectoryRoot;
		private Config.Directory DirectorySchema;

		private bool Initialized = false;

		private class Entry
		{
			public String FullPath;
			public int Depth;
			public UInt64 Size;
			public UInt64 MTime;
			public bool IsDirectory;
		}

		List<Entry> Entries;

		public UnsyncDirectoryEnumerator(Config.Project ProjectSchema, UnsyncQueryConfig Config)
		{
			this.Config = Config;
			this.ProjectSchema = ProjectSchema;
		}
		public UnsyncDirectoryEnumerator(String Path, Config.Directory DirectorySchema, UnsyncQueryConfig Config)
		{
			this.Config = Config;
			this.DirectoryRoot = Path;
			this.DirectorySchema = DirectorySchema;
		}

		public string FormatArtifactPath(string virtualPath) 
		{
			if (Config.server?.protocol == "horde")
			{
				// extract artifact ID assuming 'foobar#1234abcd' convention
				int artifactIdPos = virtualPath.LastIndexOf('#');
				string artifactId = artifactIdPos == -1 ? virtualPath : virtualPath.Substring(artifactIdPos);				
				return artifactId;
			}
			else
			{
				return virtualPath; 
			}
		}

		private async Task LazyInit(CancellationToken cancellationToken)
		{
			if (!Initialized)
			{
				Entries = new List<Entry>();

				if (ProjectSchema != null)
				{
					await InitProject(cancellationToken);
				}
				else if (DirectorySchema != null)
				{
					await InitDirectory(cancellationToken);
				}
				else
				{
					throw new Exception("Unexpected UnsyncDirectoryEnumerator configuration. Either project or directory must be specified.");
				}
			}
		}
		class BuildTraversalState
		{
			public bool FoundCL = false;
			public bool FoundStream = false;
		}

		private void ProcessQueryResult(SearchQueryResult queryResult)
		{
			foreach (var queryEntry in queryResult.entries)
			{
				Entry entry = new Entry();
				entry.FullPath = Path.Combine(queryResult.root, queryEntry.path);
				entry.Depth = GetDirectoryDepth(entry.FullPath);
				entry.IsDirectory = queryEntry.is_directory;
				entry.MTime = queryEntry.mtime;
				entry.Size = queryEntry.size;
				Entries.Add(entry);
			}
		}

		private async Task RunQueries(List<String> queryStrings, CancellationToken cancellationToken)
		{
			foreach (String query in queryStrings)
			{
				String argsStr = query + $" {Config.server.GetCommandLineArgs()}";

				var proc = new AsyncProcess(Config.unsyncPath, argsStr);
				var responseJson = "";
				// TODO: read stderr stream and somehow report status/errors
				await foreach (var str in proc.RunAsync(cancellationToken, false /*ReadStdErr*/))
				{
					responseJson += str;
				}

				if (proc.ExitCode == 0)
				{
					try
					{
						SearchQueryResult queryResult = JsonSerializer.Deserialize<SearchQueryResult>(responseJson);
						ProcessQueryResult(queryResult);
					}
					catch (Exception ex)
					{
						App.Current.LogError("Exception during unsync query: " + ex.Message);
					}
				}
			}
		}

		private async Task InitProject(CancellationToken cancellationToken)
		{
			String baseQuery = $"query search \"{ProjectSchema.Root}\"";

			BuildTraversalState traversalState = new BuildTraversalState();
			List<String> queryStrings = new List<String>();

			foreach (Config.Directory childDir in ProjectSchema.Children)
			{
				GenerateQueries(ref traversalState, childDir, baseQuery, ref queryStrings);
			}

			await RunQueries(queryStrings, cancellationToken);

			Initialized = true;
		}

		private void GenerateQueries(ref BuildTraversalState traversalState, Config.Directory dir, String query, ref List<String> output)
		{
			if (traversalState != null)
			{
				if (dir.CL != null)
				{
					traversalState.FoundCL = true;
				}

				if (dir.Stream != null)
				{
					traversalState.FoundStream = true;
				}

				if (traversalState.FoundCL && traversalState.FoundStream)
				{
					// Stop visiting directories early when rules for matching CL and Stream are found in the project config.
					// Typically this just includes the project root directory, but if the builds are organized
					// by Stream and then by CL, we want to traverse a deeper hierarchy.

					output.Add(query);

					return;
				}
			}

			// Adding a regex rule for the directory will instruct unsync to list all of its children
			query += $" \"{dir.Regex}\"";

			if (dir.SubDirectories.Any())
			{
				foreach (Config.Directory childDir in dir.SubDirectories)
				{
					GenerateQueries(ref traversalState, childDir, query, ref output);
				}
			}
			else
			{
				output.Add(query);
			}
		}

		private async Task InitDirectory(CancellationToken cancellationToken)
		{
			String baseQuery = $"query search \"{DirectoryRoot}\"";

			BuildTraversalState traversalState = null;
			List<String> queryStrings = new List<String>();

			foreach (Config.Directory childDir in DirectorySchema.SubDirectories)
			{
				GenerateQueries(ref traversalState, childDir, baseQuery, ref queryStrings);
			}

			await RunQueries(queryStrings, cancellationToken);

			Initialized = true;
		}

		private int GetDirectoryDepth(string path)
		{
			int depth = 0;
			foreach (var c in path)
			{
				if (c == Path.DirectorySeparatorChar)
				{
					depth += 1;
				}
			}
			return depth;
		}

		private struct EntryFilter
		{
			public bool IncludeFiles;
			public bool IncludeDirectories;

			public bool ShouldInclude(Entry entry)
			{
				if (entry.IsDirectory && !IncludeDirectories)
				{
					return false;
				}

				if (!entry.IsDirectory && !IncludeFiles)
				{
					return false;
				}

				return true;
			}
		}

		private Task<IEnumerable<DirectoryEntry>> EnumerateEntries(string path, CancellationToken token, EntryFilter filter)
		{
			var tcs = new TaskCompletionSource<IEnumerable<DirectoryEntry>>();
			Task.Run(async () =>
			{
				using var cancel = token.Register(() => tcs.TrySetCanceled());
				try
				{
					await LazyInit(token);
					var dirs = new List<DirectoryEntry>();

					int pathDepth = GetDirectoryDepth(path);

					string requiredPrefix = path + Path.DirectorySeparatorChar;

					// Doesn't run very often and typically matches most entries anyway,
					// so can just do a naive search without building any hierarchies.
					foreach (var entry in Entries)
					{
						if (!filter.ShouldInclude(entry))
						{
							continue;
						}

						// We want only directories that are immediate children of the given path
						if (entry.Depth == 1 + pathDepth
							&& entry.FullPath.StartsWith(requiredPrefix))
						{
							//String relativePath = Path.GetRelativePath(path, entry.FullPath);
							DateTime mtime = DateTime.FromFileTime((long)Math.Min(entry.MTime, (UInt64)DateTime.MaxValue.ToFileTime()));
							DirectoryEntry dirEntry = new DirectoryEntry 
							{ 
								Path = entry.FullPath,
								MTime = mtime,
							};
							dirs.Add(dirEntry);
						}
					}

					tcs.TrySetResult(dirs);
				}
				catch (OperationCanceledException)
				{
					tcs.TrySetCanceled();
				}
			});
			return tcs.Task;
		}

		public Task<IEnumerable<DirectoryEntry>> EnumerateDirectories(string path, CancellationToken token)
		{
			EntryFilter filter;
			filter.IncludeDirectories = true;
			filter.IncludeFiles = false;
			return EnumerateEntries(path, token, filter);
		}
		public Task<IEnumerable<DirectoryEntry>> EnumerateFiles(string path, CancellationToken token)
		{
			EntryFilter filter;
			filter.IncludeDirectories = false;
			filter.IncludeFiles = true;
			return EnumerateEntries(path, token, filter);
		}
	}

	public class SingleOrArrayConverter : JsonConverter<List<String>>
	{
		public override List<String> Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			return reader.TokenType switch
			{
				JsonTokenType.String => new List<String> { reader.GetString() },
				JsonTokenType.StartArray => JsonSerializer.Deserialize<List<String>>(ref reader, options),
				_ => throw new JsonException("Unexpected JSON format")
			};
		}

		public override void Write(Utf8JsonWriter writer, List<String> value, JsonSerializerOptions options)
		{
			if (value.Count == 1)
			{
				writer.WriteStringValue(value[0]);
			}
			else
			{
				JsonSerializer.Serialize(writer, value, options);
			}
		}
	}
}
