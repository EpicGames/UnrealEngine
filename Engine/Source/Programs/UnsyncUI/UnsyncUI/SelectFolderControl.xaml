<!--Copyright Epic Games, Inc. All Rights Reserved.-->

<UserControl x:Class="UnsyncUI.SelectFolderControl"
             xmlns="http://schemas.microsoft.com/winfx/2006/xaml/presentation"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006" 
             xmlns:d="http://schemas.microsoft.com/expression/blend/2008" 
             xmlns:local="clr-namespace:UnsyncUI"
             mc:Ignorable="d"
             d:DesignWidth="350">
    <Grid>
        <Grid.ColumnDefinitions>
            <ColumnDefinition Width="*"/>
            <ColumnDefinition Width="auto"/>
        </Grid.ColumnDefinitions>
        <TextBox 
            Grid.Column="0"
            Text="{Binding SelectedPath, Mode=TwoWay, UpdateSourceTrigger=PropertyChanged, RelativeSource={RelativeSource Mode=FindAncestor, AncestorType=UserControl}}" 
            ToolTip="{Binding Description, RelativeSource={RelativeSource Mode=FindAncestor, AncestorType=UserControl}}"/>
        <Button Grid.Column="1" Content=" ... " Click="OnBrowseClicked" ToolTip="Browse ..."/>
    </Grid>
</UserControl>
