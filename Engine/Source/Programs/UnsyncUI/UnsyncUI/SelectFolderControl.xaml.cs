// Copyright Epic Games, Inc. All Rights Reserved.

using System.Windows;
using System.Windows.Controls;
using System.Windows.Data;

namespace UnsyncUI
{
	/// <summary>
	/// Interaction logic for SelectFolderControl.xaml
	/// </summary>
	public partial class SelectFolderControl : UserControl
    {
        public static readonly DependencyProperty DescriptionProperty = DependencyProperty.Register(nameof(Description), typeof(string), typeof(SelectFolderControl));
        public static readonly DependencyProperty SelectedPathProperty = DependencyProperty.Register(nameof(SelectedPath), typeof(string), typeof(SelectFolderControl), new FrameworkPropertyMetadata()
        {
            BindsTwoWayByDefault = true,
            DefaultUpdateSourceTrigger = UpdateSourceTrigger.LostFocus
        });

        public string Description
        {
            get => (string)GetValue(DescriptionProperty);
            set => SetValue(DescriptionProperty, value);
        }

        public string SelectedPath
        {
            get => (string)GetValue(SelectedPathProperty);
            set => SetValue(SelectedPathProperty, value);
        }

        public SelectFolderControl()
        {
            InitializeComponent();
        }

        public void OnBrowseClicked(object sender, RoutedEventArgs e)
        {
            SelectedPath = Shell.SelectFolder(SelectedPath, Description);
        }
    }
}
