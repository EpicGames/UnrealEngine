// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Threading.Tasks;
using System.Windows.Forms;

namespace UnsyncUI
{
	public static class Shell
	{
		public static void LaunchExplorer(string path)
		{
			Task.Run(() => Process.Start("explorer.exe", path));
		}

		public static string SelectFolder(string root, string desc)
		{
			using FolderBrowserDialog fbd = new FolderBrowserDialog()
			{
				AutoUpgradeEnabled = true,
				RootFolder = Environment.SpecialFolder.Desktop,
				ShowNewFolderButton = true,
				SelectedPath = root,
				Description = string.IsNullOrWhiteSpace(desc) ? string.Empty : desc,
				UseDescriptionForTitle = !string.IsNullOrWhiteSpace(desc)
			};

			if (fbd.ShowDialog() == DialogResult.OK)
			{
				return fbd.SelectedPath;
			}

			return root;
		}

		public static string SelectSaveFile(string title, string filter, string defaultExt, string defaultFilename)
		{
			using SaveFileDialog sfd = new SaveFileDialog()
			{
				AddExtension = true,
				Filter = filter,
				Title = title,
				DefaultExt = defaultExt,
				FileName = defaultFilename
			};

			if (sfd.ShowDialog() == DialogResult.OK)
				return sfd.FileName;

			return null;
		}
	}
}
