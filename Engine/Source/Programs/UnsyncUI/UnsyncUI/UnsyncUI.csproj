<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />

  <PropertyGroup>
    <OutputType>WinExe</OutputType>
    <TargetFramework>net10.0-windows</TargetFramework>
    <UseWPF>true</UseWPF>
    <UseWindowsForms>true</UseWindowsForms>
    <ApplicationManifest>app.manifest</ApplicationManifest>
    <IncludeAllContentForSelfExtract>true</IncludeAllContentForSelfExtract>
    <ApplicationIcon>unsync_48px.ico</ApplicationIcon>
    <Version>$(VersionPrefix)1.0.27</Version>
  </PropertyGroup>
</Project>
