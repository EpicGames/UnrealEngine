// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using System.Threading.Tasks.Dataflow;
using System.Windows.Input;

namespace UnsyncUI
{
	public abstract class BaseModel : INotifyPropertyChanged
    {
        public event PropertyChangedEventHandler PropertyChanged;

        protected void OnPropertyChanged(string name)
            => PropertyChanged?.Invoke(this, new PropertyChangedEventArgs(name));

        protected void SetProperty<T>(ref T property, T newValue, [CallerMemberName] string name = null)
        {
            if (!Equals(property, newValue))
            {
                property = newValue;
                OnPropertyChanged(name);
            }
        }
    }

    public class Command<T> : BaseModel, ICommand
    {
        public event EventHandler CanExecuteChanged;
        private Action<T> action;
        private bool enabled = false;

        public bool Enabled
        {
            get => enabled;
            set
            {
                if (enabled != value)
                {
                    enabled = value;
                    CanExecuteChanged?.Invoke(this, new EventArgs());
					OnPropertyChanged(nameof(Enabled));
                }
            }
        }

        public Command(Action<T> action)
        {
            this.action = action;
        }

        public bool CanExecute(object parameter) => Enabled;
        public void Execute(object parameter) => action((T)parameter);
    }

    public sealed class Command : Command<object>
    {
        public Command(Action action)
            : base(o => action())
        { }
    }

	public struct DirectoryEntry
	{
		public string Path;
		public DateTime MTime;

		public static IEnumerable<string> ToPaths(IEnumerable<DirectoryEntry> dirEntries)
		{
			return dirEntries.Select(entry => entry.Path);
		}
	}

	public interface IDirectoryEnumerator
	{
		public Task<IEnumerable<DirectoryEntry>> EnumerateDirectories(string path, CancellationToken token);
		public Task<IEnumerable<DirectoryEntry>> EnumerateFiles(string path, CancellationToken token);
		public string FormatArtifactPath(string virtualPath) { return virtualPath; }
	}


	public class NativeDirectoryEnumerator : IDirectoryEnumerator
	{
		public Task<IEnumerable<DirectoryEntry>> EnumerateDirectories(string path, CancellationToken token)
		{
			return AsyncIO.EnumerateDirectoriesAsync(path, token);
		}
		public Task<IEnumerable<DirectoryEntry>> EnumerateFiles(string path, CancellationToken token)
		{
			return AsyncIO.EnumerateFilesAsync(path, token);
		}
	}

	static class AsyncIO
    {
		private static SemaphoreSlim mutex = new SemaphoreSlim(4);

        public static Task<IEnumerable<DirectoryEntry>> EnumerateDirectoriesAsync(string path, CancellationToken token)
        {
			App.Current?.LogMessage($"Enumerating: {path}");

			var tcs = new TaskCompletionSource<IEnumerable<DirectoryEntry>>();
            Task.Run(() =>
			{
				var dirInfo = new DirectoryInfo(path);

				var enumOptions = new EnumerationOptions();
				enumOptions.IgnoreInaccessible = true;
				enumOptions.AttributesToSkip = 0; // include all files (hidden and system files are skipped by default)

				using var cancel = token.Register(() => tcs.TrySetCanceled());
				try
				{
					mutex.Wait(token);
					try
					{
						var dirs = dirInfo.EnumerateDirectories("*", enumOptions);
						var dirEntries = dirs.Select(d => new DirectoryEntry { Path = d.FullName, MTime = d.LastWriteTime});
						tcs.TrySetResult(dirEntries);
					}
					catch (Exception ex)
					{
						tcs.TrySetException(ex);
					}

					mutex.Release();
				}
				catch (OperationCanceledException)
				{
					// Mutex wait was cancelled. Don't release
					tcs.TrySetCanceled();
				}
			});

            return tcs.Task;
		}
        public static Task<IEnumerable<DirectoryEntry>> EnumerateFilesAsync(string path, CancellationToken token)
        {
			var tcs = new TaskCompletionSource<IEnumerable<DirectoryEntry>>();
            Task.Run(() =>
			{
				using var cancel = token.Register(() => tcs.TrySetCanceled());
				try
				{
					mutex.Wait(token);
					try
					{
						var timer = new Stopwatch();
						timer.Start();

						var dirInfo = new DirectoryInfo(path);

						var enumOptions = new EnumerationOptions();
						enumOptions.IgnoreInaccessible = true;
						enumOptions.AttributesToSkip = 0; // include all files (hidden and system files are skipped by default)

						var dirs = dirInfo.EnumerateFiles("*", enumOptions);
						var dirEntries = dirs.Select(d => new DirectoryEntry { Path = d.FullName, MTime = d.LastWriteTime });

						timer.Stop();
						App.Current?.LogMessage($"Time: {timer.Elapsed.TotalSeconds:0.000} s - {path}");

						tcs.TrySetResult(dirEntries);
					}
					catch (Exception ex)
					{
						tcs.TrySetException(ex);
					}
					finally
					{
						mutex.Release();
					}
				}
				catch (OperationCanceledException)
				{
					// Mutex wait was cancelled. Don't release
					tcs.TrySetCanceled();
				}
			});

            return tcs.Task;
		}
    }

    public sealed class AsyncProcess
    {
		public static class ProcessTracker
		{
			[Flags]
			public enum JOBOBJECTLIMIT : uint
			{
				JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE = 0x2000
			}

			[StructLayout(LayoutKind.Sequential)]
			public struct JOBOBJECT_BASIC_LIMIT_INFORMATION
			{
				public long PerProcessUserTimeLimit;
				public long PerJobUserTimeLimit;
				public JOBOBJECTLIMIT LimitFlags;
				public UIntPtr MinimumWorkingSetSize;
				public UIntPtr MaximumWorkingSetSize;
				public uint ActiveProcessLimit;
				public long Affinity;
				public uint PriorityClass;
				public uint SchedulingClass;
			}

			[StructLayout(LayoutKind.Sequential)]
			public struct IO_COUNTERS
			{
				public ulong ReadOperationCount;
				public ulong WriteOperationCount;
				public ulong OtherOperationCount;
				public ulong ReadTransferCount;
				public ulong WriteTransferCount;
				public ulong OtherTransferCount;
			}

			[StructLayout(LayoutKind.Sequential)]
			public struct JOBOBJECT_EXTENDED_LIMIT_INFORMATION
			{
				public JOBOBJECT_BASIC_LIMIT_INFORMATION BasicLimitInformation;
				public IO_COUNTERS IoInfo;
				public UIntPtr ProcessMemoryLimit;
				public UIntPtr JobMemoryLimit;
				public UIntPtr PeakProcessMemoryUsed;
				public UIntPtr PeakJobMemoryUsed;
			}

			public enum JobObjectInfoType
			{
				ExtendedLimitInformation = 9
			}

			[DllImport("kernel32.dll", CharSet = CharSet.Unicode)]
			private extern static IntPtr CreateJobObject(IntPtr lpJobAttributes, string name);

			[DllImport("kernel32.dll")]
			private extern static bool SetInformationJobObject(IntPtr job, JobObjectInfoType infoType, IntPtr lpJobObjectInfo, uint cbJobObjectInfoLength);

			[DllImport("kernel32.dll", SetLastError = true)]
			private extern static bool AssignProcessToJobObject(IntPtr job, IntPtr process);

			private static readonly IntPtr hJob = CreateJobObject(IntPtr.Zero, $"ProcessTracker_{Process.GetCurrentProcess().Id}");

			static ProcessTracker()
			{
				if (hJob == IntPtr.Zero)
					throw new Win32Exception(Marshal.GetLastWin32Error(), "Failed to create job handle.");

				IntPtr infoExPtr = IntPtr.Zero;

				try
				{
					var size = Marshal.SizeOf(typeof(JOBOBJECT_EXTENDED_LIMIT_INFORMATION));
					infoExPtr = Marshal.AllocHGlobal(size);

					Marshal.StructureToPtr(new JOBOBJECT_EXTENDED_LIMIT_INFORMATION()
					{
						BasicLimitInformation = new JOBOBJECT_BASIC_LIMIT_INFORMATION()
						{
							LimitFlags = JOBOBJECTLIMIT.JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE
						}
					}, infoExPtr, false);

					if (!SetInformationJobObject(hJob, JobObjectInfoType.ExtendedLimitInformation, infoExPtr, (uint)size))
						throw new Win32Exception(Marshal.GetLastWin32Error(), "Failed to set job info.");
				}
				finally
				{
					if (infoExPtr != IntPtr.Zero)
						Marshal.FreeHGlobal(infoExPtr);
				}
			}

			public static void Attach(Process proc)
			{
				if (!AssignProcessToJobObject(hJob, proc.Handle) && !proc.HasExited)
					throw new Win32Exception();
			}
		}

		private Process proc = new Process();
		public int? ExitCode { get; private set; }

		public AsyncProcess(string path, string args)
		{
			proc.StartInfo = new ProcessStartInfo(path, args)
			{
				CreateNoWindow = true,
				UseShellExecute = false,
				WorkingDirectory = Environment.CurrentDirectory,
				RedirectStandardOutput = true,
				RedirectStandardError = true
			};
		}

		public enum StreamKind
		{
			StdOut,
			StdErr,
		}

		public async IAsyncEnumerable<string> RunAsync([EnumeratorCancellation] CancellationToken cancelToken, bool ReadStdErr = true)
		{
			await foreach (var (s, k) in RunAsyncStreams(cancelToken))
			{
				if (k == StreamKind.StdOut || (k==StreamKind.StdErr && ReadStdErr))
				{
					yield return s;
				}
			}
		}

		public async IAsyncEnumerable<(string, StreamKind)> RunAsyncStreams([EnumeratorCancellation] CancellationToken cancelToken)
		{
			string processFileName = Path.GetFileName(proc.StartInfo.FileName);
			App.Current?.LogMessage($"Running: {processFileName} {proc.StartInfo.Arguments}");

			try
			{
				using (var cancel = cancelToken.Register(() =>
				{
					if (!proc.HasExited)
					{
						proc.Kill();
					}
				}))
				{
					proc.Start();
					ProcessTracker.Attach(proc);

					var pipe = new BufferBlock<(string, StreamKind)>();

					async Task ReadStream(Stream stream, StreamKind Kind)
					{
						var block = new byte[4096];
						var mem = new Memory<byte>(block);

						while (true)
						{
							int bytesRead = await stream.ReadAsync(mem);
							if (bytesRead == 0)
							{
								break;
							}

							// @todo: this won't handle UTF-8 encoding if a char is split across the read boundary.
							string decodedString = Encoding.UTF8.GetString(mem.Span.Slice(0, bytesRead));
							pipe.Post((decodedString, Kind));
						}
					}

					var stdoutTask = ReadStream(proc.StandardOutput.BaseStream, StreamKind.StdOut);
					var stderrTask = ReadStream(proc.StandardError.BaseStream, StreamKind.StdErr);

					var completionTask = Task.Run(async () =>
					{
						await Task.WhenAll(stdoutTask, stderrTask);
						pipe.Complete();
					});

					while (await pipe.OutputAvailableAsync())
					{
						var (receivedString, receivedKind) = pipe.Receive();
						yield return (receivedString, receivedKind);
					}

					await completionTask;
				}
			}
			finally
			{
				ExitCode = proc.ExitCode;
				proc.Dispose();

				App.Current?.LogDebug($"Finished: {processFileName} with exit code {ExitCode}");

				cancelToken.ThrowIfCancellationRequested();
			}
		}

		public async Task<string> RunAndGetOutput(bool ReadStdErr = true)
		{
			var result = "";
			var cancellationToken = new CancellationToken();
			await foreach (var str in RunAsync(cancellationToken, ReadStdErr))
			{
				result += str;
			}
			return result;
		}
	}
}
